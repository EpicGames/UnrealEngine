// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "TextureEditorSettings.generated.h"


/**
 * Enumerates background for the texture editor view port.
 */
UENUM()
enum ETextureEditorBackgrounds : int
{
	TextureEditorBackground_SolidColor UMETA(DisplayName="Solid Color"),
	TextureEditorBackground_Checkered UMETA(DisplayName="Checkered"),
	TextureEditorBackground_CheckeredFill UMETA(DisplayName="Checkered (Fill)")
};

UENUM()
enum ETextureEditorSampling : int
{
	TextureEditorSampling_Default UMETA(DisplayName = "Default Sampling"),
	TextureEditorSampling_Point UMETA(DisplayName = "Nearest-Point Sampling"),
};

UENUM()
enum ETextureEditorVolumeViewMode : int
{
	TextureEditorVolumeViewMode_DepthSlices UMETA(DisplayName="Depth Slices"),
	TextureEditorVolumeViewMode_VolumeTrace UMETA(DisplayName="Trace Into Volume"),
};

UENUM()
enum ETextureEditorCubemapViewMode : int
{
	TextureEditorCubemapViewMode_2DView UMETA(DisplayName = "2D View"),
	TextureEditorCubemapViewMode_3DView UMETA(DisplayName = "3D View"),
};

UENUM()
enum class ETextureEditorZoomMode : uint8
{
	Custom    UMETA(DisplayName = "Specific Zoom Level"), // First so that any new modes added don't change serialized value
	Fit       UMETA(DisplayName = "Scale Down to Fit"),
	Fill      UMETA(DisplayName = "Scale to Fill"),
};

/**
 * Implements the Editor's user settings.
 */
UCLASS(MinimalAPI, config=EditorPerProjectUserSettings)
class UTextureEditorSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/** The type of background to draw in the texture editor view port. */
	UPROPERTY(config)
	TEnumAsByte<ETextureEditorBackgrounds> Background;

	/** The texture sampling mode used to render textures in the texture editor view port. */
	UPROPERTY(config)
	TEnumAsByte<ETextureEditorSampling> Sampling;

	/** The view mode when previewing volume textures. */
	UPROPERTY(config)
	TEnumAsByte<ETextureEditorVolumeViewMode> VolumeViewMode;

	/** The view mode when previewing cubemap textures. */
	UPROPERTY(config)
	TEnumAsByte<ETextureEditorCubemapViewMode> CubemapViewMode;

	/** Background and foreground color used by Texture preview view ports. */
	UPROPERTY(config, EditAnywhere, Category=Background)
	FColor BackgroundColor;

	/** The first color of the checkered background. */
	UPROPERTY(config, EditAnywhere, Category=Background)
	FColor CheckerColorOne;

	/** The second color of the checkered background. */
	UPROPERTY(config, EditAnywhere, Category=Background)
	FColor CheckerColorTwo;

	/** The size of the checkered background tiles. */
	UPROPERTY(config, EditAnywhere, Category=Background, meta=(ClampMin="2", ClampMax="4096"))
	int32 CheckerSize;
	
	/** The draw scale of the text on the viewport for infos, warnings, errors... */
	UPROPERTY(config, EditAnywhere, Category=Foreground, meta=(ClampMin="0.1"))
	float ViewportTextScale = 1.f;

public:

	/** Whether the texture should scale to fit the view port. */
	UE_DEPRECATED(4.26, "There are now more than 2 zoom modes, so this value is no longer used. Please use ZoomMode instead.")
	UPROPERTY(config)
	bool FitToViewport;

	/** Whether the texture should scale to fit or fill the view port, or use a custom zoom level. */
	UPROPERTY(config)
	ETextureEditorZoomMode ZoomMode;

	/** Color to use for the texture border, if enabled. */
	UPROPERTY(config, EditAnywhere, Category=TextureBorder)
	FColor TextureBorderColor;

	/** If true, displays a border around the texture. */
	UPROPERTY(config)
	bool TextureBorderEnabled;
};
