// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/CurveLinearColorAtlasDetailsCustomization.h"
#include "Misc/MessageDialog.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "Editor.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilder.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/STextComboBox.h"

#define LOCTEXT_NAMESPACE "FTextureDetails"


TSharedRef<IDetailCustomization> FCurveLinearColorAtlasDetails::MakeInstance()
{
	return MakeShareable(new FCurveLinearColorAtlasDetails);
}

void FCurveLinearColorAtlasDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DetailBuilder.HideCategory("Compression");
	DetailBuilder.HideCategory("LevelOfDetail");
	DetailBuilder.HideCategory("Texture");
	DetailBuilder.HideCategory("File Path");
	DetailBuilder.HideCategory("Compositing");
}

#undef LOCTEXT_NAMESPACE
