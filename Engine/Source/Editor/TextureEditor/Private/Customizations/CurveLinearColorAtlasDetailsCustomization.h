// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "UObject/WeakObjectPtr.h"
#include "Types/SlateEnums.h"
#include "TextureDetailsCustomization.h"

class IDetailLayoutBuilder;
class IPropertyHandle;

class FCurveLinearColorAtlasDetails : public FTextureDetails
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

private:
	FCurveLinearColorAtlasDetails()
	{}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

