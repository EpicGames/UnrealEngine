// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/TextureDetailsCustomization.h"

#include "AssetToolsModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Engine/Texture2D.h"
#include "IAssetTools.h"
#include "IDetailPropertyRow.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "FTextureDetails"


TSharedRef<IDetailCustomization> FTextureDetails::MakeInstance()
{
	return MakeShareable(new FTextureDetails);
}

void FTextureDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DetailBuilder.GetObjectsBeingCustomized(TexturesBeingCustomized);

	DetailBuilder.EditCategory("LevelOfDetail");
	DetailBuilder.EditCategory("Compression");
	DetailBuilder.EditCategory("Texture");
	DetailBuilder.EditCategory("Adjustments");
	DetailBuilder.EditCategory("File Path");

	OodleTextureSdkVersionPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UTexture, OodleTextureSdkVersion));
	MaxTextureSizePropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UTexture, MaxTextureSize));
	VirtualTextureStreamingPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UTexture, VirtualTextureStreaming));
		
	if( OodleTextureSdkVersionPropertyHandle->IsValidHandle() )
	{
		IDetailCategoryBuilder& CompressionCategory = DetailBuilder.EditCategory("Compression");
		IDetailPropertyRow& OodleTextureSdkVersionPropertyRow = CompressionCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UTexture, OodleTextureSdkVersion));
		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;
		FDetailWidgetRow Row;
		OodleTextureSdkVersionPropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

		const bool bShowChildren = true;
		OodleTextureSdkVersionPropertyRow.CustomWidget(bShowChildren)
			.NameContent()
			.MinDesiredWidth(Row.NameWidget.MinWidth)
			.MaxDesiredWidth(Row.NameWidget.MaxWidth)
			[
				NameWidget.ToSharedRef()
			]
			.ValueContent()
			.MinDesiredWidth(Row.ValueWidget.MinWidth)
			.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					ValueWidget.ToSharedRef()
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SButton)
					.OnClicked(this, &FTextureDetails::OnOodleTextureSdkVersionClicked)
					.ContentPadding(FMargin(2))
					.Content()
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Center)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(LOCTEXT("OodleTextureSdkVersionLatest", "latest"))
						.ToolTipText(LOCTEXT("OodleTextureSdkVersionLatestTooltip", "Update SDK Version to Latest"))
					]
				]
			];
	}
	
	// Customize MaxTextureSize
	if( MaxTextureSizePropertyHandle->IsValidHandle() && TexturesBeingCustomized.Num() == 1)
	{
		IDetailCategoryBuilder& CompressionCategory = DetailBuilder.EditCategory("Compression");
		IDetailPropertyRow& MaxTextureSizePropertyRow = CompressionCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UTexture, MaxTextureSize));
		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;
		FDetailWidgetRow Row;
		MaxTextureSizePropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

		int32 MaxTextureSize = UTexture::GetMaximumDimensionOfNonVT();

		if (UTexture* Texture = Cast<UTexture>(TexturesBeingCustomized[0].Get()))
		{
			// GetMaximumDimension is for current RHI and texture type
			MaxTextureSize = FMath::Min<int32>( Texture->GetMaximumDimension(), MaxTextureSize );
		}

		// @@ this slider is very hard to work with
		//	it's almost impossible to set low values
		// instead of being on the linear MaxTextureSize value, it should be on the log2
		//	and scaled by *10 or something
		// so the drag experience is slower and log-scaled

		const bool bShowChildren = true;
		MaxTextureSizePropertyRow.CustomWidget(bShowChildren)
			.NameContent()
			.MinDesiredWidth(Row.NameWidget.MinWidth)
			.MaxDesiredWidth(Row.NameWidget.MaxWidth)
			[
				NameWidget.ToSharedRef()
			]
			.ValueContent()
			.MinDesiredWidth(Row.ValueWidget.MinWidth)
			.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
			[
				SNew(SNumericEntryBox<int32>)
				.AllowSpin(true)
				.Value(this, &FTextureDetails::OnGetMaxTextureSize)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.MinValue(0)
				.MaxValue(MaxTextureSize)
				.MinSliderValue(0)
				.MaxSliderValue(MaxTextureSize)
				.OnValueChanged(this, &FTextureDetails::OnMaxTextureSizeChanged)
				.OnValueCommitted(this, &FTextureDetails::OnMaxTextureSizeCommitted)
				.OnBeginSliderMovement(this, &FTextureDetails::OnBeginSliderMovement)
				.OnEndSliderMovement(this, &FTextureDetails::OnEndSliderMovement)
			];
	}

	if (VirtualTextureStreamingPropertyHandle.IsValid())
	{
		DetailBuilder.HideProperty(VirtualTextureStreamingPropertyHandle);

		// Only show the option to enable VT streaming, if VT is enabled for the project.
		static const auto CVarVirtualTexturesEnabled = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VirtualTextures"));
		const static bool bVirtualTextureEnabled = CVarVirtualTexturesEnabled && CVarVirtualTexturesEnabled->GetValueOnAnyThread() != 0;

		TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
		DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
		const bool bIsValidObject = ObjectsBeingCustomized.Num() > 0;

		if (bVirtualTextureEnabled && bIsValidObject)
		{
			IDetailCategoryBuilder& TextureCategory = DetailBuilder.EditCategory("Texture");
			TextureCategory.AddCustomRow(VirtualTextureStreamingPropertyHandle->GetPropertyDisplayName())
			.NameContent()
			[
				VirtualTextureStreamingPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.VAlign(VAlign_Center)
				.TextStyle(FAppStyle::Get(), "NormalText")
				.Text_Lambda([WeakObject = ObjectsBeingCustomized[0]]()
				{
					UTexture2D const* Texture = Cast<UTexture2D>(WeakObject.Get());
					const bool bVirtualTextureStreaming = Texture != nullptr ? Texture->VirtualTextureStreaming != 0 : false;
					if (bVirtualTextureStreaming)
					{
						return LOCTEXT("Button_ConvertToRegularTexture", "Convert to Regular Texture");
					}
					else
					{
						return LOCTEXT("Button_ConvertToVirtualTexture", "Convert to Virtual Texture");
					}
				})
				.OnClicked_Lambda([WeakObject = ObjectsBeingCustomized[0]]()
				{
					if (UTexture2D* Texture = Cast<UTexture2D>(WeakObject.Get()))
					{
						if (!Texture->Source.AreAllBlocksPowerOfTwo() && Texture->PowerOfTwoMode == ETexturePowerOfTwoSetting::None)
						{
							FMessageDialog::Open(
								EAppMsgType::Ok,
								LOCTEXT("Dialog_ConvertTexturePow2_Msg", "Failed to convert a non Power of Two sized texture.\nSet a valid Padding and Resizing mode."),
								LOCTEXT("Dialog_ConvertTexture_Title", "Convert Texture"));
						}
						else
						{
							TArray<UTexture2D*> Textures;
							Textures.Add(Texture);
							IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
							AssetTools.ConvertVirtualTextures(Textures, Texture->VirtualTextureStreaming);
							return FReply::Handled();
						}
					}
					return FReply::Unhandled();
				})
			];
		}
	}
}

FReply FTextureDetails::OnOodleTextureSdkVersionClicked()
{
	for (const TWeakObjectPtr<UObject>& WeakTexture : TexturesBeingCustomized)
	{
		if (UTexture* Texture = Cast<UTexture>(WeakTexture.Get()))
		{
			// true = do Pre/PostEditChange
			Texture->UpdateOodleTextureSdkVersionToLatest(true);
		}
	}

	return FReply::Handled();
}

/** @return The value or unset if properties with multiple values are viewed */
TOptional<int32> FTextureDetails::OnGetMaxTextureSize() const
{
	int32 NumericVal;
	if (MaxTextureSizePropertyHandle->GetValue(NumericVal) == FPropertyAccess::Success)
	{
		return NumericVal;
	}

	// Return an unset value so it displays the "multiple values" indicator instead
	return TOptional<int32>();
}

void FTextureDetails::OnMaxTextureSizeChanged(int32 NewValue)
{
	if (bIsUsingSlider)
	{
		int32 OrgValue(0);
		if (MaxTextureSizePropertyHandle->GetValue(OrgValue) != FPropertyAccess::Fail)
		{
			// Value hasn't changed, so let's return now
			if (OrgValue == NewValue)
			{
				return;
			}
		}

		// We don't create a transaction for each property change when using the slider.  Only once when the slider first is moved
		// Interactive flag makes it so the texture is not rebuilt in PostEditChange
		EPropertyValueSetFlags::Type Flags = (EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable);
		MaxTextureSizePropertyHandle->SetValue(NewValue, Flags);
	}
}

void FTextureDetails::OnMaxTextureSizeCommitted(int32 NewValue, ETextCommit::Type CommitInfo)
{
	// this causes the texture to build with the new value (if necessary)
	MaxTextureSizePropertyHandle->SetValue(NewValue);
}

/**
 * Called when the slider begins to move.  We create a transaction here to undo the property
 */
void FTextureDetails::OnBeginSliderMovement()
{
	bIsUsingSlider = true;

	GEditor->BeginTransaction(TEXT("TextureDetails"), LOCTEXT("SetMaximumTextureSize", "Edit Maximum Texture Size"), nullptr /* MaxTextureSizePropertyHandle->GetProperty() */ );
}


/**
 * Called when the slider stops moving.  We end the previously created transaction
 */
void FTextureDetails::OnEndSliderMovement(int32 NewValue)
{
	bIsUsingSlider = false;

	GEditor->EndTransaction();
}


#undef LOCTEXT_NAMESPACE
