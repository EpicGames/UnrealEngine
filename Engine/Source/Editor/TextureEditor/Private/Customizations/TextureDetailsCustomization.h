// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "UObject/WeakObjectPtr.h"
#include "Types/SlateEnums.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"

class IDetailLayoutBuilder;
class IPropertyHandle;

class FTextureDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	FTextureDetails()
		: bIsUsingSlider(false)
	{}
private:
	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	TOptional<int32> OnGetMaxTextureSize() const;
	void OnMaxTextureSizeChanged(int32 NewValue);
	void OnMaxTextureSizeCommitted(int32 NewValue, ETextCommit::Type CommitInfo);
	void OnBeginSliderMovement();
	void OnEndSliderMovement(int32 NewValue);
	FReply OnOodleTextureSdkVersionClicked();

	TSharedPtr<IPropertyHandle> OodleTextureSdkVersionPropertyHandle;
	TSharedPtr<IPropertyHandle> MaxTextureSizePropertyHandle;
	TSharedPtr<IPropertyHandle> VirtualTextureStreamingPropertyHandle;
	TArray<TWeakObjectPtr<UObject>> TexturesBeingCustomized;
	bool bIsUsingSlider;
};

