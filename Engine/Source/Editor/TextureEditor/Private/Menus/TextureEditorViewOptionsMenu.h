// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Models/TextureEditorCommands.h"

#define LOCTEXT_NAMESPACE "TextureEditorViewOptionsMenu"

/**
 * Static helper class for populating the "View Options" menu in the texture editor's view port.
 */
class FTextureEditorViewOptionsMenu
{
public:

	/**
	 * Creates the menu.
	 *
	 * @param MenuBuilder The builder for the menu that owns this menu.
	 */
	static void MakeMenu(FMenuBuilder& MenuBuilder)
	{
		// view port options
		MenuBuilder.BeginSection("ViewportSection", LOCTEXT("ViewportSectionHeader", "Viewport Options"));
		{
			MenuBuilder.AddMenuEntry(FTextureEditorCommands::Get().Desaturation);

			MenuBuilder.AddSubMenu(
				LOCTEXT("Background", "Background"),
				LOCTEXT("BackgroundTooltip", "Set the viewport's background"),
				FNewMenuDelegate::CreateStatic(&FTextureEditorViewOptionsMenu::GenerateBackgroundMenuContent)
			);

			MenuBuilder.AddMenuEntry(FTextureEditorCommands::Get().TextureBorder);

			MenuBuilder.AddSubMenu(
				LOCTEXT("Sampling", "Sampling Mode"),
				LOCTEXT("SamplingTooltip", "Set the texture sampling mode"),
				FNewMenuDelegate::CreateStatic(&FTextureEditorViewOptionsMenu::GenerateSamplingMenuContent)
			);
		}
		MenuBuilder.EndSection();

		MenuBuilder.AddMenuSeparator();
		MenuBuilder.AddMenuEntry(FTextureEditorCommands::Get().Settings);
	}

protected:

	/**
	 * Creates the 'Background' sub-menu.
	 *
	 * @param MenuBuilder The builder for the menu that owns this menu.
	 */
	static void GenerateBackgroundMenuContent( FMenuBuilder& MenuBuilder )
	{
		MenuBuilder.AddMenuEntry(FTextureEditorCommands::Get().CheckeredBackground);
		MenuBuilder.AddMenuEntry(FTextureEditorCommands::Get().CheckeredBackgroundFill);
		MenuBuilder.AddMenuEntry(FTextureEditorCommands::Get().SolidBackground);
	}

	/**
	 * Creates the 'Sampling Mode' sub-menu.
	 *
	 * @param MenuBuilder The builder for the menu that owns this menu.
	 */
	static void GenerateSamplingMenuContent(FMenuBuilder& MenuBuilder)
	{
		MenuBuilder.AddMenuEntry(FTextureEditorCommands::Get().DefaultSampling);
		MenuBuilder.AddMenuEntry(FTextureEditorCommands::Get().PointSampling);
	}
};


#undef LOCTEXT_NAMESPACE
