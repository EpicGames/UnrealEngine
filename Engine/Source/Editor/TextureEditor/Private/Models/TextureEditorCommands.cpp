// Copyright Epic Games, Inc. All Rights Reserved.

#include "Models/TextureEditorCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "TextureEditorCommands"

void FTextureEditorCommands::RegisterCommands()
{
	UI_COMMAND(RedChannel, "Red", "Toggles the red channel", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(GreenChannel, "Green", "Toggles the green channel", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BlueChannel, "Blue", "Toggles the blue channel", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(AlphaChannel, "Alpha", "Toggles the alpha channel", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(Desaturation, "Desaturation", "Toggles color desaturation", EUserInterfaceActionType::ToggleButton, FInputChord());
	
	UI_COMMAND(CheckeredBackground, "Checkered", "Checkered background pattern behind the texture", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CheckeredBackgroundFill, "Checkered (Fill)", "Checkered background pattern behind the entire viewport", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(FitToViewport, "Scale To Fit", "Scales the texture down to fit within the viewport if needed", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(FillToViewport, "Scale To Fill", "Scales the texture to fill the viewport", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ZoomToNatural, "Scale To 100%", "Resets the zoom level to 100%, displaying the texture at its natural size", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SolidBackground, "Solid Color", "Solid color background", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(TextureBorder, "Draw Border", "If enabled, a border is drawn around the texture", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(DefaultSampling, "Default Sampling", "Use default texture sampling", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(PointSampling, "Nearest-Point Sampling", "Use nearest-point texture sampling", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(CompressNow, "Compress", "Compress the texture", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(Reimport, "Reimport", "Reimports the texture from file", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(Settings, "Advanced Settings...", "Opens the advanced settings for the texture editor", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
