// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

/**
 * Holds the UI commands for the TextureEditorToolkit widget.
 */
class FTextureEditorCommands
	: public TCommands<FTextureEditorCommands>
{
public:

	/**
	 * Default constructor.
	 */
	FTextureEditorCommands( ) 
		: TCommands<FTextureEditorCommands>("TextureEditor", NSLOCTEXT("Contexts", "TextureEditor", "Texture Editor"), NAME_None, FAppStyle::GetAppStyleSetName())
	{ }

public:

	// TCommands interface

	virtual void RegisterCommands() override;
	
public:

	/** Toggles the red channel */
	TSharedPtr<FUICommandInfo> RedChannel;
	
	/** Toggles the green channel */
	TSharedPtr<FUICommandInfo> GreenChannel;
	
	/** Toggles the blue channel */
	TSharedPtr<FUICommandInfo> BlueChannel;
	
	/** Toggles the alpha channel */
	TSharedPtr<FUICommandInfo> AlphaChannel;

	/** Toggles color saturation */
	TSharedPtr<FUICommandInfo> Desaturation;

	/** If enabled, the texture will be scaled down to fit within the viewport if needed */
	TSharedPtr<FUICommandInfo> FitToViewport;

	/** If enabled, the texture will be scaled to fill the viewport */
	TSharedPtr<FUICommandInfo> FillToViewport;

	/** If enabled, the texture will be zoomed back to its size (zoom level 100%) */
	TSharedPtr<FUICommandInfo> ZoomToNatural;

	/** Sets the checkered background pattern */
	TSharedPtr<FUICommandInfo> CheckeredBackground;

	/** Sets the checkered background pattern (filling the view port) */
	TSharedPtr<FUICommandInfo> CheckeredBackgroundFill;

	/** Sets the solid color background */
	TSharedPtr<FUICommandInfo> SolidBackground;

	/** If enabled, a border is drawn around the texture */
	TSharedPtr<FUICommandInfo> TextureBorder;

	/** If enabled, default sampling is used for rendering the texture */
	TSharedPtr<FUICommandInfo> DefaultSampling;

	/** If enabled, nearest-point sampling is used for rendering the texture */
	TSharedPtr<FUICommandInfo> PointSampling;

	/** Compress the texture */
	TSharedPtr<FUICommandInfo> CompressNow;

	/** Reimports the texture */
	TSharedPtr<FUICommandInfo> Reimport;

	/** Open the texture editor settings. */
	TSharedPtr< FUICommandInfo > Settings;
};
