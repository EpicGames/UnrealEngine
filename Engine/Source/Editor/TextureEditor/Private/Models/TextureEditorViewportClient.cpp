// Copyright Epic Games, Inc. All Rights Reserved.

#include "Models/TextureEditorViewportClient.h"
#include "Widgets/Layout/SScrollBar.h"
#include "CanvasItem.h"
#include "EngineFontServices.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Texture2D.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Engine/TextureCube.h"
#include "Engine/Texture2DArray.h"
#include "Engine/TextureCubeArray.h"
#include "Engine/VolumeTexture.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "Engine/TextureRenderTargetCube.h"
#include "Engine/TextureRenderTargetVolume.h"
#include "UnrealEdGlobals.h"
#include "CubemapUnwrapUtils.h"
#include "Slate/SceneViewport.h"
#include "Texture2DPreview.h"
#include "VolumeTexturePreview.h"
#include "TextureEncodingSettings.h"
#include "TextureEditorSettings.h"
#include "Widgets/STextureEditorViewport.h"
#include "CanvasTypes.h"
#include "ImageUtils.h"
#include "EngineUtils.h"
#include "EngineModule.h"
#include "RendererInterface.h"
#include "TextureResource.h"

static TAutoConsoleVariable<int32> CVarEnableVTFeedback(
	TEXT("r.VT.UpdateFeedbackTextureEditor"),
	1,
	TEXT("Enable/Disable the CPU feedback analysis in the texture editor."),
	ECVF_RenderThreadSafe
);

struct FTextureErrorLogger : public FOutputDevice
{
	UTexture* TextureToMonitor = nullptr;
	FCriticalSection LogLinesLock; // Locks RelevantLogLings since Serialize can be called while we are drawing the lines.
	TArray<TPair<bool, FString>> RelevantLogLines;

	double CaptureUntilTime = 0;
	bool bCurrentlyCapturing = false;

	FTextureErrorLogger(UTexture* InTextureToMonitor)
	{
		TextureToMonitor = InTextureToMonitor;
		GLog->AddOutputDevice(this);
	}
	~FTextureErrorLogger()
	{
		GLog->RemoveOutputDevice(this);
	}
	FTextureErrorLogger(FTextureErrorLogger&&) = delete;
	FTextureErrorLogger(FTextureErrorLogger&) = delete;
	FTextureErrorLogger& operator=(FTextureErrorLogger&) = delete;
	FTextureErrorLogger& operator=(FTextureErrorLogger&&) = delete;


	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		if (TextureToMonitor == nullptr)
		{
			return;
		}


		// Here we aren't capturing yet.
		// See if the string relates to us.
		if (FCString::Stristr(V, *TextureToMonitor->GetName()))
		{
			// It turns out we can't rely on a startup message with the new build flow (IBuild), so
			// we just start capturing if we ever see a warning or error with our name in it.
			// 
			// If it's any of the launching messages then we clear our backlog.
			if (Verbosity == ELogVerbosity::Error || 
				Verbosity == ELogVerbosity::Warning)
			{
				bCurrentlyCapturing = true;
			}

			// If it's "Building textures" then we started a new build and need to empty our list.
			// For shared linear we also might only get a "Tiling" message... but we don't want to clear
			// on that one since we'll likely get it after the Building textures message
			if (FCString::Stristr(V, TEXT("Building textures")))
			{
				FScopeLock _(&LogLinesLock);
				RelevantLogLines.Empty();
				bCurrentlyCapturing = true;
			}
			static FName TextureBuildFunctionCategory = "LogTextureBuildFunction";
			if (Category == TextureBuildFunctionCategory)
			{
				if (FCString::Stristr(V, TEXT("Compressing")))
				{
					FScopeLock _(&LogLinesLock);
					RelevantLogLines.Empty();
					bCurrentlyCapturing = true;
				}
			}
		}

		//
		// Error messages don't reliably put the texture name in the messages, and we don't necessarily know
		// that the error will come from a texture category if it's something like bulk data. However, we do
		// ~generally~ expect that if we have a texture editor open, the only texture that's getting built is
		// the one we are editing. So we just capture all errors/warning between the time we see "building texture"
		// for ourselves and when the texture async build is complete.
		if (bCurrentlyCapturing)
		{
			// There's a race condition here because the log messages take a while to get to us, so an
			// async texture build can finish well before we see any messages about it, so we keep watching for a second.
			if (CaptureUntilTime)
			{
				double CurrentTime = FPlatformTime::Seconds();
				if (CurrentTime > CaptureUntilTime)
				{
					bCurrentlyCapturing = false;
					CaptureUntilTime = 0;
					return;
				}
			}
			else if (TextureToMonitor->IsAsyncCacheComplete())
			{
				// Logs might not have gotten to us, so we keep capturing for another second.
				CaptureUntilTime = FPlatformTime::Seconds() + 1;
			}

			// Add any errors or warnings to the list
			if (Verbosity == ELogVerbosity::Error ||
				Verbosity == ELogVerbosity::Warning)
			{
				FScopeLock _(&LogLinesLock);
				if (Verbosity == ELogVerbosity::Error)
				{
					RelevantLogLines.Add(TPair<bool, FString>(true, FString(V)));
				}
				else if (Verbosity == ELogVerbosity::Warning)
				{
					RelevantLogLines.Add(TPair<bool, FString>(false, FString(V)));
				}
				if (RelevantLogLines.Num() == 10)
				{
					RelevantLogLines.Add(TPair<bool, FString>(false, TEXT("Too much to show: check Output Log")));
					bCurrentlyCapturing = false;
					CaptureUntilTime = 0;
				}
			}
			return;
		}

	}
};

/* FTextureEditorViewportClient structors
 *****************************************************************************/

FTextureEditorViewportClient::FTextureEditorViewportClient( TWeakPtr<ITextureEditorToolkit> InTextureEditor, TWeakPtr<STextureEditorViewport> InTextureEditorViewport )
	: TextureEditorPtr(InTextureEditor)
	, TextureEditorViewportPtr(InTextureEditorViewport)
	, CheckerboardTexture(nullptr)
{
	check(TextureEditorPtr.IsValid() && TextureEditorViewportPtr.IsValid());

	ModifyCheckerboardTextureColors();

	TextureConsoleCapture = MakeUnique<FTextureErrorLogger>(InTextureEditor.Pin()->GetTexture());
}


FTextureEditorViewportClient::~FTextureEditorViewportClient( )
{
	DestroyCheckerboardTexture();
}


/* FViewportClient interface
 *****************************************************************************/

void FTextureEditorViewportClient::Draw(FViewport* Viewport, FCanvas* Canvas)
{
	if (!TextureEditorPtr.IsValid())
	{
		return;
	}

	TSharedPtr<ITextureEditorToolkit> TextureEditorPinned = TextureEditorPtr.Pin();
	const UTextureEditorSettings& Settings = *GetDefault<UTextureEditorSettings>();

	UTexture* Texture = TextureEditorPinned->GetTexture();
	FVector2D Ratio = FVector2D(GetViewportHorizontalScrollBarRatio(), GetViewportVerticalScrollBarRatio());
	FVector2D ViewportSize = FVector2D(TextureEditorViewportPtr.Pin()->GetViewport()->GetSizeXY());
	FVector2D ScrollBarPos = GetViewportScrollBarPositions();
	int32 BorderSize = Settings.TextureBorderEnabled ? 1 : 0;
	float YOffset = static_cast<float>((Ratio.Y > 1.0) ? ((ViewportSize.Y - (ViewportSize.Y / Ratio.Y)) * 0.5) : 0);
	int32 YPos = (int32)FMath::Clamp(FMath::RoundToInt(YOffset - ScrollBarPos.Y + BorderSize), TNumericLimits<int32>::Min(), TNumericLimits<int32>::Max());
	float XOffset = static_cast<float>((Ratio.X > 1.0) ? ((ViewportSize.X - (ViewportSize.X / Ratio.X)) * 0.5) : 0);
	int32 XPos = (int32)FMath::Clamp(FMath::RoundToInt(XOffset - ScrollBarPos.X + BorderSize), TNumericLimits<int32>::Min(), TNumericLimits<int32>::Max());
	
	UpdateScrollBars();

	Canvas->Clear( Settings.BackgroundColor );
	
	UFont* ReportingFont = GEngine->GetLargeFont();
	constexpr double ReportingLineSpacing = 2;
	double ReportingLineX = 8;
	double ReportingLineY = 8;
	float FontScale = 1.f;

	checkf(ReportingFont, TEXT("Texture Editor : Engine large font must be set"));

	FSlateFontInfo FontInfo(ReportingFont, ReportingFont->LegacyFontSize);
	double ReportingLineHeight = 0;
	{
		const UTextureEditorSettings* TextureEditorSettings = GetDefault<UTextureEditorSettings>();
		const TSharedPtr<FSlateFontCache> FontCache = FEngineFontServices::Get().GetFontCache();
		if (TextureEditorSettings && FontCache)
		{
			FontScale = FMath::Max(0.1f, TextureEditorSettings->ViewportTextScale);
			ReportingLineHeight = FontCache->GetCharacterList(FontInfo, FontScale).GetMaxHeight();
		}
		else
		{
			ReportingLineHeight = FMath::CeilToInt(ReportingFont->GetMaxCharHeight());
		}
		ReportingLineHeight += ReportingLineSpacing;
	}
	
	auto DrawText = [&FontInfo, &Canvas, &FontScale](const double LineX, const double LineY, const FText& InText, const FLinearColor& InColor)->FVector2D
		{
			const FVector2D InPosition(LineX, LineY);
			FCanvasTextItem TextItem(InPosition, InText, FontInfo, InColor);
			TextItem.Scale = FVector2D(FontScale);
			Canvas->DrawItem(TextItem);
			return TextItem.DrawnSize;
		};

	// make sure error messages are shown last, no matter if we return early or not
	ON_SCOPE_EXIT {
		// Print any warnings/errors that we saw in the output log.
		FScopeLock _(&TextureConsoleCapture->LogLinesLock);
		for (TPair<bool, FString>& ReportedLine : TextureConsoleCapture->RelevantLogLines)
		{
			ReportingLineY += DrawText(ReportingLineX, ReportingLineY, FText::FromString(ReportedLine.Value), ReportedLine.Key ? FLinearColor::Red : FLinearColor::Yellow).Y;
			ReportingLineY += ReportingLineSpacing;
		}
	};

	if ( Texture->IsCompiling() )
	{
		const FText Message = NSLOCTEXT("TextureEditor", "Compiling", "Compiling...");
		DrawText(ReportingLineX, ReportingLineY, Message, FLinearColor::White);
		ReportingLineY += ReportingLineHeight;
		return;
	}

	UTexture2D* Texture2D = Cast<UTexture2D>(Texture);
	UTextureCube* TextureCube = Cast<UTextureCube>(Texture);
	UTexture2DArray* Texture2DArray = Cast<UTexture2DArray>(Texture);
	UTextureCubeArray* TextureCubeArray = Cast<UTextureCubeArray>(Texture);
	UVolumeTexture* VolumeTexture = Cast<UVolumeTexture>(Texture);
	UTextureRenderTarget2D* TextureRT2D = Cast<UTextureRenderTarget2D>(Texture);
	UTextureRenderTarget2DArray* TextureRT2DArray = Cast<UTextureRenderTarget2DArray>(Texture);
	UTextureRenderTargetCube* RTTextureCube = Cast<UTextureRenderTargetCube>(Texture);
	UTextureRenderTargetVolume* RTTextureVolume = Cast<UTextureRenderTargetVolume>(Texture);

	// Fully stream in the texture before drawing it.
	Texture->SetForceMipLevelsToBeResident(30.0f);
	Texture->WaitForStreaming();

	TextureEditorPinned->PopulateQuickInfo();

	// Figure out the size we need
	int32 Width, Height, Depth, ArraySize;
	TextureEditorPinned->CalculateTextureDimensions(Width, Height, Depth, ArraySize, false);
	const float MipLevel = (float)TextureEditorPinned->GetMipLevel();
	const float LayerIndex = (float)TextureEditorPinned->GetLayer();
	const float SliceIndex = (float)TextureEditorPinned->GetSlice();
	const bool bUsePointSampling = TextureEditorPinned->GetSampling() == ETextureEditorSampling::TextureEditorSampling_Point;

	bool bIsVirtualTexture = false;

	UTexture2D* CPUCopyTexture = nullptr;
	if (Texture2D)
	{
		CPUCopyTexture = Texture2D->GetCPUCopyTexture();
		if (CPUCopyTexture)
		{
			FIntPoint CenteringOffset(0, 0);
			CenteringOffset += Viewport->GetSizeXY() / 2;
			CenteringOffset -= FIntPoint(Width, Height) / 2;

			Canvas->DrawTile(CenteringOffset.X, CenteringOffset.Y, Width, Height, 0.0f, 0.0f, 1.0f, 1.0f, FLinearColor::White, CPUCopyTexture->GetResource());
			return;
		}
	}
	
	TRefCountPtr<FBatchedElementParameters> BatchedElementParameters;

	if (GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM5)
	{
		if (TextureCube || TextureCubeArray || RTTextureCube)
		{
			const int32 FaceIndex = TextureEditorPinned->GetFace();
			const FVector4f& IdentityW = FVector4f(FVector3f::ZeroVector, 1);
			// when the face index is not specified, generate a view matrix based on the tracked orientation in the texture editor
			// (assuming that identity matrix displays the face 0 properly oriented in space),
			// otherwise generate a view matrix which displays the selected face using DDS orientation and face order
			// (positive x, negative x, positive y, negative y, positive z, negative z)
			const FMatrix44f& ViewMatrix =
				FaceIndex < 0 ? FMatrix44f(FRotationMatrix::Make(TextureEditorPinned->GetOrientation())) :
				FaceIndex == 0 ? FMatrix44f(FVector3f::ForwardVector, FVector3f::DownVector, FVector3f::RightVector, IdentityW) :
				FaceIndex == 1 ? FMatrix44f(FVector3f::BackwardVector, FVector3f::UpVector, FVector3f::RightVector, IdentityW) :
				FaceIndex == 2 ? FMatrix44f(FVector3f::RightVector, FVector3f::ForwardVector, FVector3f::DownVector, IdentityW) :
				FaceIndex == 3 ? FMatrix44f(FVector3f::LeftVector, FVector3f::ForwardVector, FVector3f::UpVector, IdentityW) :
				FaceIndex == 4 ? FMatrix44f(FVector3f::UpVector, FVector3f::ForwardVector, FVector3f::RightVector, IdentityW) :
				FMatrix44f(FVector3f::DownVector, FVector3f::BackwardVector, FVector3f::RightVector, IdentityW);
			const bool bShowLongLatUnwrap = TextureEditorPinned->GetCubemapViewMode() == TextureEditorCubemapViewMode_2DView && FaceIndex < 0;
			BatchedElementParameters = new FMipLevelBatchedElementParameters(MipLevel, SliceIndex, TextureCubeArray != nullptr, ViewMatrix, bShowLongLatUnwrap, false, bUsePointSampling);
		}
		else if (VolumeTexture)
		{
			BatchedElementParameters = new FBatchedElementVolumeTexturePreviewParameters(
				TextureEditorPinned->GetVolumeViewMode() == TextureEditorVolumeViewMode_DepthSlices,
				MipLevel, 
				(float)TextureEditorPinned->GetVolumeOpacity(),
				true, 
				TextureEditorPinned->GetOrientation(),
				bUsePointSampling);
		}
		else if (RTTextureVolume)
		{
			BatchedElementParameters = new FBatchedElementVolumeTexturePreviewParameters(
				TextureEditorPinned->GetVolumeViewMode() == TextureEditorVolumeViewMode_DepthSlices,
				MipLevel,
				(float)TextureEditorPinned->GetVolumeOpacity(),
				true,
				TextureEditorPinned->GetOrientation(),
				bUsePointSampling);
		}
		else if (Texture2D)
		{
			bool bIsNormalMap = Texture2D->IsNormalMap();
			bool bIsSingleChannel = Texture2D->CompressionSettings == TC_Grayscale || Texture2D->CompressionSettings == TC_Alpha;
			bool bSingleVTPhysicalSpace = Texture2D->IsVirtualTexturedWithSinglePhysicalSpace();
			bIsVirtualTexture = Texture2D->IsCurrentlyVirtualTextured();
			BatchedElementParameters = new FBatchedElementTexture2DPreviewParameters(MipLevel, LayerIndex, SliceIndex, bIsNormalMap, bIsSingleChannel, bSingleVTPhysicalSpace, bIsVirtualTexture, false, bUsePointSampling);
		}
		else if (Texture2DArray) 
		{
			bool bIsNormalMap = Texture2DArray->IsNormalMap();
			bool bIsSingleChannel = Texture2DArray->CompressionSettings == TC_Grayscale || Texture2DArray->CompressionSettings == TC_Alpha;
			BatchedElementParameters = new FBatchedElementTexture2DPreviewParameters(MipLevel, LayerIndex, SliceIndex, bIsNormalMap, bIsSingleChannel, false, false, true, bUsePointSampling);
		}
		else if (TextureRT2D)
		{
			BatchedElementParameters = new FBatchedElementTexture2DPreviewParameters(MipLevel, LayerIndex, SliceIndex, false, false, false, false, false, bUsePointSampling);
		}
		else if (TextureRT2DArray)
		{
			BatchedElementParameters = new FBatchedElementTexture2DPreviewParameters(MipLevel, LayerIndex, SliceIndex, false, false, false, false, true, bUsePointSampling);
		}
		else
		{
			// Default to treating any UTexture derivative as a 2D texture resource
			BatchedElementParameters = new FBatchedElementTexture2DPreviewParameters(MipLevel, LayerIndex, SliceIndex, false, false, false, false, false, bUsePointSampling);
		}
	}

	// Draw the background checkerboard pattern in the same size/position as the render texture so it will show up anywhere
	// the texture has transparency
	if (Viewport && CheckerboardTexture)
	{
		const float CheckerboardSizeX = (float)FMath::Max<int32>(1, CheckerboardTexture->GetSizeX());
		const float CheckerboardSizeY = (float)FMath::Max<int32>(1, CheckerboardTexture->GetSizeY());
		if (Settings.Background == TextureEditorBackground_CheckeredFill)
		{
			Canvas->DrawTile( 0.0f, 0.0f, Viewport->GetSizeXY().X, Viewport->GetSizeXY().Y, 0.0f, 0.0f, (float)Viewport->GetSizeXY().X / CheckerboardSizeX, (float)Viewport->GetSizeXY().Y / CheckerboardSizeY, FLinearColor::White, CheckerboardTexture->GetResource());
		}
		else if (Settings.Background == TextureEditorBackground_Checkered)
		{
			Canvas->DrawTile( XPos, YPos, Width, Height, 0.0f, 0.0f, (float)Width / CheckerboardSizeX, (float)Height / CheckerboardSizeY, FLinearColor::White, CheckerboardTexture->GetResource());
		}
	}

	FTexturePlatformData** RunningPlatformDataPtr = Texture->GetRunningPlatformData();
	float Exposure = RunningPlatformDataPtr && *RunningPlatformDataPtr && IsHDR((*RunningPlatformDataPtr)->PixelFormat) ? FMath::Pow(2.0f, (float)TextureEditorPinned->GetExposureBias()) : 1.0f;


	if ( Texture->GetResource() != nullptr && !CPUCopyTexture )
	{
		FCanvasTileItem TileItem( FVector2D( XPos, YPos ), Texture->GetResource(), FVector2D( Width, Height ), FLinearColor(Exposure, Exposure, Exposure) );
		TileItem.BlendMode = TextureEditorPinned->GetColourChannelBlendMode();
		TileItem.BatchedElementParameters = BatchedElementParameters;

		if (bIsVirtualTexture && Texture->Source.GetNumBlocks() > 1)
		{
			// Adjust UVs to display entire UDIM range, accounting for UE inverted V-axis
			const FIntPoint BlockSize = Texture->Source.GetSizeInBlocks();
			TileItem.UV0 = FVector2D(0.0f, 1.0f - (float)BlockSize.Y);
			TileItem.UV1 = FVector2D((float)BlockSize.X, 1.0f);
		}

		Canvas->DrawItem( TileItem );

		// Draw a white border around the texture to show its extents
		if (Settings.TextureBorderEnabled)
		{
			float ScaledBorderSize = ((float)BorderSize - 1) * 0.5f;
			FCanvasBoxItem BoxItem(FVector2D((float)XPos - ScaledBorderSize, (float)YPos - ScaledBorderSize), FVector2D(Width + BorderSize, Height + BorderSize));
			BoxItem.LineThickness = (float)BorderSize;
			BoxItem.SetColor( Settings.TextureBorderColor );
			Canvas->DrawItem( BoxItem );
		}

		// if we are presenting a virtual texture, make the appropriate tiles resident
		if (bIsVirtualTexture && CVarEnableVTFeedback.GetValueOnAnyThread() != 0)
		{
			FVirtualTexture2DResource* VTResource = static_cast<FVirtualTexture2DResource*>(Texture->GetResource());
			const FVector2D ScreenSpaceSize((float)Width, (float)Height);
			const FVector2D ViewportPositon((float)XPos, (float)YPos);
			const FVector2D UV0 = TileItem.UV0;
			const FVector2D UV1 = TileItem.UV1;

			UE::RenderCommandPipe::FSyncScope SyncScope;

			const ERHIFeatureLevel::Type InFeatureLevel = GMaxRHIFeatureLevel;
			ENQUEUE_RENDER_COMMAND(MakeTilesResident)(
				[InFeatureLevel, VTResource, ScreenSpaceSize, ViewportPositon, ViewportSize, UV0, UV1, MipLevel](FRHICommandListImmediate& RHICmdList)
			{
				// AcquireAllocatedVT() must happen on render thread
				IAllocatedVirtualTexture* AllocatedVT = VTResource->AcquireAllocatedVT();

				IRendererModule& RenderModule = GetRendererModule();
				RenderModule.RequestVirtualTextureTiles(AllocatedVT, ScreenSpaceSize, ViewportPositon, ViewportSize, UV0, UV1, (int32)MipLevel);
				RenderModule.LoadPendingVirtualTextureTiles(RHICmdList, InFeatureLevel);
			});
		}
	}

	// If we are requesting an explicit mip level of a VT asset, test to see if we can even display it properly and warn about it
	if (bIsVirtualTexture && MipLevel >= 0.f)
	{
		const uint32 Mip = (uint32)MipLevel;
		const FIntPoint SizeOnMip = { Texture2D->GetSizeX() >> Mip,Texture2D->GetSizeY() >> Mip };
		const uint64 NumPixels = static_cast<uint64>(SizeOnMip.X) * SizeOnMip.Y;

		const FVirtualTexture2DResource* Resource = (FVirtualTexture2DResource*)Texture2D->GetResource();
		const FIntPoint PhysicalTextureSize = Resource->GetPhysicalTextureSize(0u);
		const uint64 NumPhysicalPixels = static_cast<uint64>(PhysicalTextureSize.X) * PhysicalTextureSize.Y;

		if (NumPixels >= NumPhysicalPixels)
		{
			const FText Message = NSLOCTEXT("TextureEditor", "InvalidVirtualTextureMipDisplay", "Displaying a virtual texture on a mip level that is larger than the physical cache. Rendering will probably be invalid!");
			DrawText(ReportingLineX, ReportingLineY, Message, FLinearColor::Red);
			ReportingLineY += ReportingLineHeight;
		}
	}

	// If we have compression deferred, make it clear they are viewing unencoded data.
	if (Texture->DeferCompression)
	{
		const FText Message = NSLOCTEXT("TextureEditor", "CompressionDeferred", "Compression Deferred: Viewing unencoded data!");
		DrawText(ReportingLineX, ReportingLineY, Message, FLinearColor::Yellow);
		ReportingLineY += ReportingLineHeight;
	}
	else
	{
		// Check if we are viewing an encoding that isn't Final.
		FTexturePlatformData** PlatformDataPtr = Texture->GetRunningPlatformData();
		
		if (PlatformDataPtr &&
			PlatformDataPtr[0] && // Can be null if we haven't had a chance to call CachePlatformData on the texture (brand new)
			PlatformDataPtr[0]->ResultMetadata.bIsValid)
		{
			FResolvedTextureEncodingSettings const& EncodeSettings = FResolvedTextureEncodingSettings::Get();
			bool bEncodingDiffers = (EncodeSettings.Project.bFastUsesRDO != EncodeSettings.Project.bFinalUsesRDO ||
				EncodeSettings.Project.FastEffortLevel != EncodeSettings.Project.FinalEffortLevel ||
				EncodeSettings.Project.FastRDOLambda != EncodeSettings.Project.FinalRDOLambda ||
				EncodeSettings.Project.FastUniversalTiling != EncodeSettings.Project.FinalUniversalTiling);

			if (PlatformDataPtr[0]->ResultMetadata.bWasEditorCustomEncoding)
			{
				const FText LeadInText = NSLOCTEXT("TextureEditor", "ViewingCustom", "Viewing custom encoding");
				DrawText(ReportingLineX, ReportingLineY, LeadInText, FLinearColor::Yellow);
				ReportingLineY += ReportingLineHeight;
			}
			else if (bEncodingDiffers &&
				PlatformDataPtr[0]->ResultMetadata.bSupportsEncodeSpeed &&
				PlatformDataPtr[0]->ResultMetadata.EncodeSpeed != (uint8)ETextureEncodeSpeed::Final)
			{
				// We aren't final - which might not matter if they encode the same way, so just show the differences from final.
				double CurrentX = ReportingLineX;

				auto DrawComma = [&DrawText, &CurrentX, &ReportingLineY]()
				{
					CurrentX += DrawText(CurrentX, ReportingLineY, FText::FromString(TEXT(", ")), FLinearColor::Yellow).X;
				};

				const FText LeadInText = NSLOCTEXT("TextureEditor", "EncodingDifference", "Viewing non-shipping encoding, differences are: ");
				CurrentX += DrawText(CurrentX, ReportingLineY, LeadInText, FLinearColor::Yellow).X;

				const FText HelpText = NSLOCTEXT("TextureEditor", "ShowFinal", "Check \"Editor Show Final Encoding\" to see shipping encoding.");
				DrawText(ReportingLineX, ReportingLineY + ReportingLineHeight, HelpText, FLinearColor::Yellow);

				bool bNeedComma = false;
				if (EncodeSettings.Project.bFastUsesRDO != EncodeSettings.Project.bFinalUsesRDO)
				{
					const FText RDOText = NSLOCTEXT("TextureEditor", "RDODifference", "RDO On/Off");
					CurrentX += DrawText(CurrentX, ReportingLineY, RDOText, FLinearColor::Yellow).X;
					bNeedComma = true;
				}
				else if (EncodeSettings.Project.bFinalUsesRDO)
				{
					// Some stuff only matters if RDO is on
					if (EncodeSettings.Project.FastRDOLambda != EncodeSettings.Project.FinalRDOLambda)
					{
						if (bNeedComma) 
						{
							DrawComma();
						}
						const FText RDOText = NSLOCTEXT("TextureEditor", "RDOLambdaDifference", "RDO Lambda");
						CurrentX += DrawText(CurrentX, ReportingLineY, RDOText, FLinearColor::Yellow).X;
						bNeedComma = true;
					}
					if (EncodeSettings.Project.FastUniversalTiling != EncodeSettings.Project.FinalUniversalTiling)
					{
						if (bNeedComma)
						{
							DrawComma();
						}
						const FText EffortText = NSLOCTEXT("TextureEditor", "UTDifference", "RDO Universal Tiling");
						CurrentX += DrawText(CurrentX, ReportingLineY, EffortText, FLinearColor::Yellow).X;
						bNeedComma = true;
					}
				}
				if (EncodeSettings.Project.FastEffortLevel != EncodeSettings.Project.FinalEffortLevel)
				{
					if (bNeedComma)
					{
						DrawComma();
					}
					const FText EffortText = NSLOCTEXT("TextureEditor", "EffortDifference", "Encode Effort");
					CurrentX += DrawText(CurrentX, ReportingLineY, EffortText, FLinearColor::Yellow).X;
					bNeedComma = true;
				}

				ReportingLineY += 2*ReportingLineHeight;
			} // end if difference exists
		} // end if valid result metadata
	} // end if not deferring

}


bool FTextureEditorViewportClient::InputKey(const FInputKeyEventArgs& InEventArgs)
{
	if (InEventArgs.Event == IE_Pressed)
	{
		if (InEventArgs.Key == EKeys::MouseScrollUp)
		{
			TextureEditorPtr.Pin()->ZoomIn();

			return true;
		}
		else if (InEventArgs.Key == EKeys::MouseScrollDown)
		{
			TextureEditorPtr.Pin()->ZoomOut();

			return true;
		}
		else if (InEventArgs.Key == EKeys::MiddleMouseButton && TextureEditorPtr.Pin()->IsUsingOrientation())
		{
			TextureEditorPtr.Pin()->ResetOrientation();
		}
	}
	return false;
}

bool FTextureEditorViewportClient::InputAxis(const FInputKeyEventArgs& InEventArgs)
{
	const FKey& Key = InEventArgs.Key;
	FViewport* Viewport = InEventArgs.Viewport;
	const float Delta = InEventArgs.AmountDepressed;
	
	if (Key == EKeys::MouseX || Key == EKeys::MouseY)
	{
		TSharedPtr<ITextureEditorToolkit> TextureEditorPinned = TextureEditorPtr.Pin();
		if (TextureEditorPinned->IsUsingOrientation() && Viewport->KeyState(EKeys::LeftMouseButton))
		{
			const float RotationSpeed = .2f;
			FRotator DeltaOrientation = FRotator(Key == EKeys::MouseY ? Delta * RotationSpeed : 0, Key == EKeys::MouseX ? Delta * RotationSpeed : 0, 0);
			if (TextureEditorPinned->IsVolumeTexture())
			{
				TextureEditorPinned->SetOrientation((FRotationMatrix::Make(DeltaOrientation) * FRotationMatrix::Make(TextureEditorPinned->GetOrientation())).Rotator());
			}
			else
			{
				TextureEditorPinned->SetOrientation(TextureEditorPinned->GetOrientation() + DeltaOrientation);
			}
		}
		else if (ShouldUseMousePanning(Viewport))
		{
			TSharedPtr<STextureEditorViewport> EditorViewport = TextureEditorViewportPtr.Pin();

			int32 Width, Height, Depth, ArraySize;
			TextureEditorPinned->CalculateTextureDimensions(Width, Height, Depth, ArraySize, true);

			if (Key == EKeys::MouseY)
			{
				float VDistFromBottom = EditorViewport->GetVerticalScrollBar()->DistanceFromBottom();
				float VRatio = GetViewportVerticalScrollBarRatio();
				float localDelta = (Delta / static_cast<float>(Height));
				EditorViewport->GetVerticalScrollBar()->SetState(FMath::Clamp((1.f - VDistFromBottom - VRatio) + localDelta, 0.0f, 1.0f - VRatio), VRatio);
			}
			else
			{
				float HDistFromBottom = EditorViewport->GetHorizontalScrollBar()->DistanceFromBottom();
				float HRatio = GetViewportHorizontalScrollBarRatio();
				float localDelta = (Delta / static_cast<float>(Width)) * -1.f; // delta needs to be inversed
				EditorViewport->GetHorizontalScrollBar()->SetState(FMath::Clamp((1.f - HDistFromBottom - HRatio) + localDelta, 0.0f, 1.0f - HRatio), HRatio);
			}
		}
		return true;
	}

	return false;
}

bool FTextureEditorViewportClient::ShouldUseMousePanning(FViewport* Viewport) const
{
	if (Viewport->KeyState(EKeys::RightMouseButton))
	{
		TSharedPtr<STextureEditorViewport> EditorViewport = TextureEditorViewportPtr.Pin();
		return EditorViewport.IsValid() && EditorViewport->GetVerticalScrollBar().IsValid() && EditorViewport->GetHorizontalScrollBar().IsValid();
	}

	return false;
}

EMouseCursor::Type FTextureEditorViewportClient::GetCursor(FViewport* Viewport, int32 X, int32 Y)
{
	return ShouldUseMousePanning(Viewport) ? EMouseCursor::GrabHandClosed : EMouseCursor::Default;
}

bool FTextureEditorViewportClient::InputGesture(FViewport* Viewport, const FInputDeviceId DeviceId, EGestureEvent GestureType, EGesturePhase GesturePhase, const FVector2D& GestureDelta, bool bIsDirectionInvertedFromDevice, const uint64 Timestamp)
{
	const bool LeftMouseButtonDown = Viewport->KeyState(EKeys::LeftMouseButton);
	const bool RightMouseButtonDown = Viewport->KeyState(EKeys::RightMouseButton);

	if (GestureType == EGestureEvent::Scroll && !LeftMouseButtonDown && !RightMouseButtonDown)
	{
		double CurrentZoom = TextureEditorPtr.Pin()->GetCustomZoomLevel();
		TextureEditorPtr.Pin()->SetCustomZoomLevel(CurrentZoom + GestureDelta.Y * 0.01);
		return true;
	}

	return false;
}


void FTextureEditorViewportClient::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(CheckerboardTexture);
}


void FTextureEditorViewportClient::ModifyCheckerboardTextureColors()
{
	DestroyCheckerboardTexture();

	const UTextureEditorSettings& Settings = *GetDefault<UTextureEditorSettings>();
	CheckerboardTexture = FImageUtils::CreateCheckerboardTexture(Settings.CheckerColorOne, Settings.CheckerColorTwo, Settings.CheckerSize);
}


FText FTextureEditorViewportClient::GetDisplayedResolution() const
{
	// Zero is the default size 
	int32 Height, Width, Depth, ArraySize;
	TextureEditorPtr.Pin()->CalculateTextureDimensions(Width, Height, Depth, ArraySize, false);

	FText CubemapInfo;
	UTexture* Texture = TextureEditorPtr.Pin()->GetTexture();
	if (Texture->IsA(UTextureCube::StaticClass()) || Texture->IsA(UTextureCubeArray::StaticClass()) || Texture->IsA(UTextureRenderTargetCube::StaticClass()))
	{
		CubemapInfo = NSLOCTEXT("TextureEditor", "DisplayedPerCubeSide", "*6 (CubeMap)");
	}

	FNumberFormattingOptions Options;
	Options.UseGrouping = false;

	if (Depth > 0)
	{
		return FText::Format(NSLOCTEXT("TextureEditor", "DisplayedResolutionThreeDimension", "Displayed: {0}x{1}x{2}"), FText::AsNumber(Width, &Options), FText::AsNumber(Height, &Options), FText::AsNumber(Depth, &Options));
	}
	else if (ArraySize > 0)
	{
		return FText::Format(NSLOCTEXT("TextureEditor", "DisplayedResolution", "Displayed: {0}x{1}{2}*{3}"), FText::AsNumber(Width, &Options), FText::AsNumber(Height, &Options), CubemapInfo, FText::AsNumber(ArraySize, &Options));
	}
	else
	{
		return FText::Format(NSLOCTEXT("TextureEditor", "DisplayedResolutionTwoDimension", "Displayed: {0}x{1}{2}"), FText::AsNumber(Width, &Options), FText::AsNumber(Height, &Options), CubemapInfo);
	}
}


float FTextureEditorViewportClient::GetViewportVerticalScrollBarRatio() const
{
	int32 Height = 1;
	int32 Width = 1;
	float WidgetHeight = 1.0f;
	if (TextureEditorViewportPtr.Pin()->GetVerticalScrollBar().IsValid())
	{
		int32 Depth, ArraySize;
		TextureEditorPtr.Pin()->CalculateTextureDimensions(Width, Height, Depth, ArraySize, true);

		WidgetHeight = (float)(TextureEditorViewportPtr.Pin()->GetViewport()->GetSizeXY().Y);
	}

	return WidgetHeight / (float)Height;
}


float FTextureEditorViewportClient::GetViewportHorizontalScrollBarRatio() const
{
	int32 Width = 1;
	int32 Height = 1;
	float WidgetWidth = 1.0f;
	if (TextureEditorViewportPtr.Pin()->GetHorizontalScrollBar().IsValid())
	{
		int32 Depth, ArraySize;
		TextureEditorPtr.Pin()->CalculateTextureDimensions(Width, Height, Depth, ArraySize, true);

		WidgetWidth = (float)(TextureEditorViewportPtr.Pin()->GetViewport()->GetSizeXY().X);
	}

	return WidgetWidth / (float)Width;
}


void FTextureEditorViewportClient::UpdateScrollBars()
{
	TSharedPtr<STextureEditorViewport> Viewport = TextureEditorViewportPtr.Pin();

	if (!Viewport.IsValid() || !Viewport->GetVerticalScrollBar().IsValid() || !Viewport->GetHorizontalScrollBar().IsValid())
	{
		return;
	}

	float VRatio = GetViewportVerticalScrollBarRatio();
	float HRatio = GetViewportHorizontalScrollBarRatio();
	float VDistFromTop = Viewport->GetVerticalScrollBar()->DistanceFromTop();
	float VDistFromBottom = Viewport->GetVerticalScrollBar()->DistanceFromBottom();
	float HDistFromTop = Viewport->GetHorizontalScrollBar()->DistanceFromTop();
	float HDistFromBottom = Viewport->GetHorizontalScrollBar()->DistanceFromBottom();

	if (VRatio < 1.0f)
	{
		if (VDistFromBottom < 1.0f)
		{
			Viewport->GetVerticalScrollBar()->SetState(FMath::Clamp((1.0f + VDistFromTop - VDistFromBottom - VRatio) * 0.5f, 0.0f, 1.0f - VRatio), VRatio);
		}
		else
		{
			Viewport->GetVerticalScrollBar()->SetState(0.0f, VRatio);
		}
	}

	if (HRatio < 1.0f)
	{
		if (HDistFromBottom < 1.0f)
		{
			Viewport->GetHorizontalScrollBar()->SetState(FMath::Clamp((1.0f + HDistFromTop - HDistFromBottom - HRatio) * 0.5f, 0.0f, 1.0f - HRatio), HRatio);
		}
		else
		{
			Viewport->GetHorizontalScrollBar()->SetState(0.0f, HRatio);
		}
	}
}


FVector2D FTextureEditorViewportClient::GetViewportScrollBarPositions() const
{
	FVector2D Positions = FVector2D::ZeroVector;
	if (TextureEditorViewportPtr.Pin()->GetVerticalScrollBar().IsValid() && TextureEditorViewportPtr.Pin()->GetHorizontalScrollBar().IsValid())
	{
		int32 Width, Height, Depth, ArraySize;
		UTexture* Texture = TextureEditorPtr.Pin()->GetTexture();
		float VRatio = GetViewportVerticalScrollBarRatio();
		float HRatio = GetViewportHorizontalScrollBarRatio();
		float VDistFromTop = TextureEditorViewportPtr.Pin()->GetVerticalScrollBar()->DistanceFromTop();
		float VDistFromBottom = TextureEditorViewportPtr.Pin()->GetVerticalScrollBar()->DistanceFromBottom();
		float HDistFromTop = TextureEditorViewportPtr.Pin()->GetHorizontalScrollBar()->DistanceFromTop();
		float HDistFromBottom = TextureEditorViewportPtr.Pin()->GetHorizontalScrollBar()->DistanceFromBottom();
	
		TextureEditorPtr.Pin()->CalculateTextureDimensions(Width, Height, Depth, ArraySize, true);

		if ((TextureEditorViewportPtr.Pin()->GetVerticalScrollBar()->GetVisibility() == EVisibility::Visible) && VDistFromBottom < 1.0f)
		{
			Positions.Y = FMath::Clamp((1.0f + VDistFromTop - VDistFromBottom - VRatio) * 0.5f, 0.0f, 1.0f - VRatio) * (float)Height;
		}
		else
		{
			Positions.Y = 0.0f;
		}

		if ((TextureEditorViewportPtr.Pin()->GetHorizontalScrollBar()->GetVisibility() == EVisibility::Visible) && HDistFromBottom < 1.0f)
		{
			Positions.X = FMath::Clamp((1.0f + HDistFromTop - HDistFromBottom - HRatio) * 0.5f, 0.0f, 1.0f - HRatio) * (float)Width;
		}
		else
		{
			Positions.X = 0.0f;
		}
	}

	return Positions;
}

void FTextureEditorViewportClient::DestroyCheckerboardTexture()
{
	if (CheckerboardTexture)
	{
		if (CheckerboardTexture->GetResource())
		{
			CheckerboardTexture->ReleaseResource();
		}
		CheckerboardTexture->MarkAsGarbage();
		CheckerboardTexture = NULL;
	}
}