// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "UObject/GCObject.h"
#include "UnrealClient.h"
#include "ViewportClient.h"

class FCanvas;
class ITextureEditorToolkit;
class STextureEditorViewport;
class UTexture2D;

class FTextureEditorViewportClient
	: public FViewportClient
	, public FGCObject
{
public:
	/** Constructor */
	FTextureEditorViewportClient(TWeakPtr<ITextureEditorToolkit> InTextureEditor, TWeakPtr<STextureEditorViewport> InTextureEditorViewport);
	virtual ~FTextureEditorViewportClient() override;

	/** FViewportClient interface */
	virtual void Draw(FViewport* Viewport, FCanvas* Canvas) override;
	virtual bool InputKey(const FInputKeyEventArgs& InEventArgs) override;
	virtual bool InputAxis(const FInputKeyEventArgs& InEventArgs) override;
	virtual bool InputGesture(FViewport* Viewport, const FInputDeviceId DeviceId, EGestureEvent GestureType, EGesturePhase GesturePhase, const FVector2D& GestureDelta, bool bIsDirectionInvertedFromDevice, const uint64 Timestamp) override;
	virtual UWorld* GetWorld() const override { return nullptr; }
	virtual EMouseCursor::Type GetCursor(FViewport* Viewport, int32 X, int32 Y) override;

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FTextureEditorViewportClient");
	}

	/** Modifies the checkerboard texture's data */
	void ModifyCheckerboardTextureColors();

	/** Returns a string representation of the currently displayed textures resolution */
	FText GetDisplayedResolution() const;

	/** Returns the ratio of the size of the Texture texture to the size of the viewport */
	float GetViewportVerticalScrollBarRatio() const;
	float GetViewportHorizontalScrollBarRatio() const;

private:
	/** Updates the states of the scrollbars */
	void UpdateScrollBars();

	/** Returns the positions of the scrollbars relative to the Texture textures */
	FVector2D GetViewportScrollBarPositions() const;

	/** Destroy the checkerboard texture if one exists */
	void DestroyCheckerboardTexture();

	/** TRUE if right clicking and dragging for panning a texture 2D */
	bool ShouldUseMousePanning(FViewport* Viewport) const;

private:
	/** Pointer back to the Texture editor tool that owns us */
	TWeakPtr<ITextureEditorToolkit> TextureEditorPtr;

	/** Pointer back to the Texture viewport control that owns us */
	TWeakPtr<STextureEditorViewport> TextureEditorViewportPtr;

	/** Checkerboard texture */
	TObjectPtr<UTexture2D> CheckerboardTexture;

	/** Output device to monitor the output log for anything relevant to us.*/
	TUniquePtr<struct FTextureErrorLogger> TextureConsoleCapture;
};
