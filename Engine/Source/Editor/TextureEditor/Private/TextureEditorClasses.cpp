// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Color.h"
#include "TextureEditorSettings.h"
#include "UObject/Object.h"

class FObjectInitializer;


UTextureEditorSettings::UTextureEditorSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
	, Background(TextureEditorBackground_Checkered)
	, Sampling(TextureEditorSampling_Default)
	, BackgroundColor(FColor::FromHex("242424FF"))
	, CheckerColorOne(FColor(128, 128, 128))
	, CheckerColorTwo(FColor(64, 64, 64))
	, CheckerSize(16)
	, FitToViewport(true)
	, ZoomMode(ETextureEditorZoomMode::Fit)
	, TextureBorderColor(FColor::White)
	, TextureBorderEnabled(true)
{ }
