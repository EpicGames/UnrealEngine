// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// MaxZoom and MinZoom should both be powers of two
const double MaxZoom = 16.0;
const double MinZoom = 1.0/64;
// ZoomFactor is multiplicative such that an integer number of steps will give a power of two zoom (50% or 200%)
const int ZoomFactorLogSteps = 8;
const double ZoomFactor = pow(2.0,1.0/ZoomFactorLogSteps);

// Specifies the maximum allowed exposure bias.
const int32 MaxExposure = 10;

// Specifies the minimum allowed exposure bias.
const int32 MinExposure = -10;
