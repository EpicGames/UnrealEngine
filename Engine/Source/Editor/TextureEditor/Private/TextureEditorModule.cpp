// Copyright Epic Games, Inc. All Rights Reserved.


#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Engine/Texture.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Interfaces/ITextureEditorToolkit.h"
#include "Interfaces/ITextureEditorModule.h"
#include "PropertyEditorModule.h"
#include "TextureEditorSettings.h"
#include "TextureEditorToolkit.h"
#include "ISettingsModule.h"
#include "Customizations/TextureDetailsCustomization.h"
#include "Customizations/CurveLinearColorAtlasDetailsCustomization.h"


#define LOCTEXT_NAMESPACE "FTextureEditorModule"


extern const FName TextureEditorAppIdentifier = FName(TEXT("TextureEditorApp"));


/*-----------------------------------------------------------------------------
   FTextureEditorModule
-----------------------------------------------------------------------------*/

class FTextureEditorModule
	: public ITextureEditorModule
{
public:

	// ITextureEditorModule interface

	virtual TSharedRef<ITextureEditorToolkit> CreateTextureEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UTexture* Texture ) override
	{
		TSharedRef<FTextureEditorToolkit> NewTextureEditor(new FTextureEditorToolkit());
		Texture->bMarkAsEditorStreamingPool = true;
		NewTextureEditor->InitTextureEditor(Mode, InitToolkitHost, Texture);

		return NewTextureEditor;
	}

	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager( ) override
	{
		return MenuExtensibilityManager;
	}

	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager( ) override
	{
		return ToolBarExtensibilityManager;
	}

public:

	// IModuleInterface interface

	virtual void StartupModule( ) override
	{
		// register menu extensions
		MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
		ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

		// register settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->RegisterSettings("Editor", "ContentEditors", "TextureEditor",
				LOCTEXT("TextureEditorSettingsName", "Texture Editor"),
				LOCTEXT("TextureEditorSettingsDescription", "Configure the look and feel of the Texture Editor."),
				GetMutableDefault<UTextureEditorSettings>()
			);
		}

		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout("Texture", FOnGetDetailCustomizationInstance::CreateStatic(&FTextureDetails::MakeInstance));
		PropertyModule.RegisterCustomClassLayout("CurveLinearColorAtlas", FOnGetDetailCustomizationInstance::CreateStatic(&FCurveLinearColorAtlasDetails::MakeInstance));
	}

	virtual void ShutdownModule( ) override
	{
		// unregister settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Editor", "ContentEditors", "TextureEditor");
		}

		// unregister menu extensions
		MenuExtensibilityManager.Reset();
		ToolBarExtensibilityManager.Reset();

		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomClassLayout("Texture");
		PropertyModule.UnregisterCustomClassLayout("CurveLinearColorAtlas");
	}

private:

	// Holds the menu extensibility manager.
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;

	// Holds the tool bar extensibility manager.
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
};


IMPLEMENT_MODULE(FTextureEditorModule, TextureEditor);


#undef LOCTEXT_NAMESPACE
