// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureEditorToolkit.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "Editor.h"
#include "Widgets/Layout/SBorder.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "EditorReimportHandler.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Engine/LightMapTexture2D.h"
#include "Engine/ShadowMapTexture2D.h"
#include "Engine/Texture2DDynamic.h"
#include "Engine/TextureCube.h"
#include "Engine/Texture2DArray.h"
#include "Engine/TextureCubeArray.h"
#include "Engine/VolumeTexture.h"
#include "TextureEncodingSettings.h"
#include "Engine/TextureRenderTarget.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "Engine/TextureRenderTargetCube.h"
#include "Engine/TextureRenderTargetVolume.h"
#include "Interfaces/IProjectManager.h"
#include "Interfaces/ITextureEditorModule.h"
#include "TextureEditor.h"
#include "Slate/SceneViewport.h"
#include "PropertyEditorModule.h"
#include "TextureEditorConstants.h"
#include "Models/TextureEditorCommands.h"
#include "Widgets/STextureEditorViewport.h"
#include "ISettingsModule.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "Curves/CurveLinearColorAtlas.h"
#include "TextureEditorSettings.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Menus/TextureEditorViewOptionsMenu.h"
#include "MediaTexture.h"
#include "TextureEncodingSettings.h"
#include "SEnumCombo.h"
#include "Widgets/Layout/SHeader.h"
#include "DerivedDataCacheKey.h"
#include "Settings/ProjectPackagingSettings.h"
#include "Compression/OodleDataCompressionUtil.h"
#include "Components/SceneCaptureComponent2D.h"
#include "TextureResource.h"
#include "TextureCompressorModule.h"
#include "TextureBuildUtilities.h"
#include "TextureFormatManager.h"

#define LOCTEXT_NAMESPACE "FTextureEditorToolkit"

DEFINE_LOG_CATEGORY_STATIC(LogTextureEditor, Log, All);

#define MIPLEVEL_MIN 0
#define MIPLEVEL_MAX 15
#define EXPOSURE_MIN -10
#define EXPOSURE_MAX 10


const FName FTextureEditorToolkit::ViewportTabId(TEXT("TextureEditor_Viewport"));
const FName FTextureEditorToolkit::PropertiesTabId(TEXT("TextureEditor_Properties"));
const FName FTextureEditorToolkit::OodleTabId(TEXT("TextureEditor_Oodle"));
const FName FTextureEditorToolkit::PlatformsTabId(TEXT("TextureEditor_Platforms"));

namespace
{
	const FName PlatformsColumnId_Platform(TEXT("PlatformNameCol"));
	const FName PlatformsColumnId_TextureFormat(TEXT("TextureFormatNameCol"));
	const FName PlatformsColumnId_PixelFormat(TEXT("PixelFormatCol"));
	const FName PlatformsColumnId_LODBias(TEXT("LODBiasCol"));
	const FName PlatformsColumnId_Dims(TEXT("DimsCol"));
	const FName PlatformsColumnId_NumMips(TEXT("NumMipsCol"));
	const FName PlatformsColumnId_SizeOptional(TEXT("MipSizesCol_Optional"));
	const FName PlatformsColumnId_SizeStreaming(TEXT("MipSizesCol_Streaming"));
	const FName PlatformsColumnId_SizeInline(TEXT("MipSizesCol_Inline"));
}

UNREALED_API void GetBestFitForNumberOfTiles(int32 InSize, int32& OutRatioX, int32& OutRatioY);

static EPixelFormatChannelFlags GetPixelFormatChannelFlagForButton(ETextureChannelButton InButton)
{
	switch (InButton)
	{
		case ETextureChannelButton::Red:
		{
			return EPixelFormatChannelFlags::R;
		}
		case ETextureChannelButton::Green:
		{
			return EPixelFormatChannelFlags::G;
		}
		case ETextureChannelButton::Blue:
		{
			return EPixelFormatChannelFlags::B;
		}
		case ETextureChannelButton::Alpha:
		{
			return EPixelFormatChannelFlags::A;
		}
		default:
		{
			check(false);
		}
	}

	return EPixelFormatChannelFlags::None;
}

// returns true if you should call PostEditChange/UpdateResource/etc to re-compress the texture
//	 after changing properties ; if false the PlatformData should be left as-is
static bool CanRecompressTexture(UTexture * Texture)
{
	if ( ! Texture->Source.IsValid() )
	{
		return false;
	}

	// legacy code was doing this to identify rendertargets :
	// not sure this is doing anything useful, the source.isvalid check above does all the work
	ETextureClass TextureClass = Texture->GetTextureClass();
	if (TextureClass == ETextureClass::RenderTarget ||
		TextureClass == ETextureClass::Other2DNoSource ||
		TextureClass == ETextureClass::TwoDDynamic)
	{
		return false;
	}

	return true;
}

void FTextureEditorToolkit::PostTextureRecode()
{
	if ( ! CanRecompressTexture(Texture) )
	{
		return;
	}

	// Each time we change a custom encode setting we want to re-encode the texture
	// as though we changed a compression setting on the actual texture, so we just
	// post a CompressionSettings property changed event to handle all of that for
	// us.
	// @@ this is a bit odd, why is it done this way rather than just calling PostEditChange() ?
	FProperty* Property = FindFProperty<FProperty>(UTexture::StaticClass(), "CompressionSettings");
	FPropertyChangedEvent PropertyChangedEvent(Property);
	Texture->PostEditChangeProperty(PropertyChangedEvent);

	// Clear the key we have so we know when we have new data
	OodleCompressedPreviewDDCKey.Set<FString>(FString());
}


/* FTextureEditorToolkit structors
 *****************************************************************************/

FTextureEditorToolkit::FTextureEditorToolkit()
	: Texture(nullptr)
	, VolumeOpacity(1.f)
{
}

FTextureEditorToolkit::~FTextureEditorToolkit( )
{
	// Release the VT page table allocation used to display this texture
	UTexture2D* Texture2D = Cast<UTexture2D>(Texture);
	if (Texture2D && Texture2D->IsCurrentlyVirtualTextured())
	{
		FVirtualTexture2DResource* Resource = (FVirtualTexture2DResource*)Texture2D->GetResource();
		if (Resource)
		{
			Resource->ReleaseAllocatedVT();
		}
	}

	FReimportManager::Instance()->OnPreReimport().RemoveAll(this);
	FReimportManager::Instance()->OnPostReimport().RemoveAll(this);
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.RemoveAll(this);

	GEditor->UnregisterForUndo(this);
	
	// we are leaving the texture editor
	// restore any temporary encoding settings we may have changed
	Texture->bMarkAsEditorStreamingPool = false;

	if ( Texture->DeferCompression || 
		Texture->OverrideRunningPlatformName != NAME_None ||
		CustomEncoding->bUseCustomEncode )
	{
		Texture->BlockOnAnyAsyncBuild(); // PreEditChange , but don't mark as modified ; same as Modify(false)
	
		Texture->DeferCompression = false;

		// we could leave OverrideRunningPlatformName set (like CompressFinal)
		//	it will stay set for this Editor session, it is not serialized
		// for now let's clear it for consistency's sake
		Texture->OverrideRunningPlatformName = NAME_None;

		CustomEncoding->bUseCustomEncode = false;
		Texture->TextureEditorCustomEncoding = nullptr;
	
		// Texture->CompressFinal intentionally not changed
		//	it will stay set for this Editor session, it is not serialized
		
		if ( CanRecompressTexture(Texture) )
		{
			Texture->PostEditChange();
		}
	}
}


/* FAssetEditorToolkit interface
 *****************************************************************************/

FString FTextureEditorToolkit::GetDocumentationLink( ) const 
{
	return FString(TEXT("Engine/Content/Types/Textures/Properties/Interface"));
}


void FTextureEditorToolkit::RegisterTabSpawners( const TSharedRef<class FTabManager>& InTabManager )
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_TextureEditor", "Texture Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(ViewportTabId, FOnSpawnTab::CreateSP(this, &FTextureEditorToolkit::HandleTabSpawnerSpawnViewport))
		.SetDisplayName(LOCTEXT("ViewportTab", "Viewport"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"))
		.SetReadOnlyBehavior(ETabReadOnlyBehavior::Custom);

	InTabManager->RegisterTabSpawner(PropertiesTabId, FOnSpawnTab::CreateSP(this, &FTextureEditorToolkit::HandleTabSpawnerSpawnProperties))
		.SetDisplayName(LOCTEXT("PropertiesTab", "Details") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"))
		.SetReadOnlyBehavior(ETabReadOnlyBehavior::Custom);

	InTabManager->RegisterTabSpawner(OodleTabId, FOnSpawnTab::CreateSP(this, &FTextureEditorToolkit::HandleTabSpawnerSpawnOodle))
		.SetDisplayName(LOCTEXT("OodleTab", "Oodle"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(PlatformsTabId, FOnSpawnTab::CreateSP(this, &FTextureEditorToolkit::HandleTabSpawnerSpawnPlatforms))
		.SetDisplayName(LOCTEXT("PlatformsTab", "Platforms"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
}


void FTextureEditorToolkit::UnregisterTabSpawners( const TSharedRef<class FTabManager>& InTabManager )
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(ViewportTabId);
	InTabManager->UnregisterTabSpawner(PropertiesTabId);
	InTabManager->UnregisterTabSpawner(OodleTabId);
	InTabManager->UnregisterTabSpawner(PlatformsTabId);
}


void FTextureEditorToolkit::InitTextureEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit )
{
	FReimportManager::Instance()->OnPreReimport().AddRaw(this, &FTextureEditorToolkit::HandleReimportManagerPreReimport);
	FReimportManager::Instance()->OnPostReimport().AddRaw(this, &FTextureEditorToolkit::HandleReimportManagerPostReimport);
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.AddRaw(this, &FTextureEditorToolkit::HandleAssetPostImport);

	Texture = CastChecked<UTexture>(ObjectToEdit);

	// The texture being edited might still be compiling, wait till it finishes then.
	// @@ is this necessary? can we remove it?
	Texture->BlockOnAnyAsyncBuild();

	// Support undo/redo
	Texture->SetFlags(RF_Transactional);
	GEditor->RegisterForUndo(this);

	CustomEncoding = MakeShared<FTextureEditorCustomEncode>(FTextureEditorCustomEncode());
	
	check( CustomEncoding->bUseCustomEncode == false );

	// OpenAssetEditor should ensure that we never have two toolkits open for the same asset!
	check(Texture->TextureEditorCustomEncoding == nullptr);

	// We save this as a separate object so that the engine can reference the type
	// without needing the texture editor module.
	Texture->TextureEditorCustomEncoding = CustomEncoding;

	// initialize view options
	bIsRedChannel = true;
	bIsGreenChannel = true;
	bIsBlueChannel = true;
	bIsAlphaChannel = false;

	ExposureBias = 0;

	bIsVolumeTexture = IsVolumeTexture();

	switch (Texture->CompressionSettings)
	{
	default:
		bIsAlphaChannel = !Texture->CompressionNoAlpha;
		break;
	case TC_Normalmap:
	case TC_Grayscale:
	case TC_Displacementmap:
	case TC_VectorDisplacementmap:
	case TC_DistanceFieldFont:
		bIsAlphaChannel = false;
		break;
	}

	// Start out with alpha channel unchecked for render targets.  Scene capture render targets have a depth mask
	// in the alpha channel, which causes them to appear as fully transparent (checkerboard), and it isn't easily
	// discoverable that toggling off the alpha channel would solve the issue, so it's helpful to start out with
	// it disabled.
	if (Texture->GetTextureClass() == ETextureClass::RenderTarget)
	{
		bIsAlphaChannel = false;
	}

	bIsDesaturation = false;

	PreviewEffectiveTextureWidth = 0;
	PreviewEffectiveTextureHeight = 0;

	SpecifiedMipLevel = 0;
	bUseSpecifiedMipLevel = false;

	SpecifiedLayer = 0;

	SpecifiedSlice = 0;
	bUseSpecifiedSlice = false;

	SpecifiedFace = 0;
	bUseSpecifiedFace = false;

	// Start at whatever the last used values of the following settings were
	const UTextureEditorSettings& Settings = *GetDefault<UTextureEditorSettings>();
	ZoomMode = Settings.ZoomMode;
	VolumeViewMode = Settings.VolumeViewMode;
	CubemapViewMode = Settings.CubemapViewMode;
	Sampling = Settings.Sampling;

	ResetOrientation();
	Zoom = 1.0f;

	// Register our commands. This will only register them if not previously registered
	FTextureEditorCommands::Register();

	BindCommands();
	CreateInternalWidgets();

	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_TextureEditor_Layout_v7")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()
						->SetOrientation(Orient_Vertical)
						->Split
						(
							FTabManager::NewStack()
								->AddTab(ViewportTabId, ETabState::OpenedTab)
								->SetHideTabWell(true)
								->SetSizeCoefficient(0.9f)
						)
				)
				->Split
				(
					FTabManager::NewStack()
						->AddTab(PropertiesTabId, ETabState::OpenedTab)
						->AddTab(OodleTabId, ETabState::OpenedTab)
						->AddTab(PlatformsTabId, ETabState::OpenedTab)
						->SetForegroundTab(PropertiesTabId)
						->SetSizeCoefficient(0.33f)
				)
		);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;

	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, TextureEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectToEdit);
	
	ITextureEditorModule* TextureEditorModule = &FModuleManager::LoadModuleChecked<ITextureEditorModule>("TextureEditor");
	AddMenuExtender(TextureEditorModule->GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	TexturePropertiesWidget->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateLambda([this]
	{
		return GetOpenMethod() == EAssetOpenMethod::Edit;
	}));
	
	ExtendToolBar();

	RegenerateMenusAndToolbars();

	// If we're opening the texture editor then we likely want to see the Final
	// texture quality for evaluation and parameter adjustment. This ends up latching
	// for the current editor session, but since it's already getting built this is
	// fine. This doesn't need the normal Modify() / PreEditChange/PostEditChange incantations
	// because it's transient, and any async build was completed above.
	Texture->CompressFinal = ( Texture->Source.IsValid() && ( Texture->Source.GetTotalTopMipPixelCount() <= (int64)4096*4096  ) );

	PostTextureRecode();

	// @todo toolkit world centric editing
	/*if(IsWorldCentricAssetEditor())
	{
		SpawnToolkitTab(GetToolbarTabId(), FString(), EToolkitTabSpot::ToolBar);
		SpawnToolkitTab(ViewportTabId, FString(), EToolkitTabSpot::Viewport);
		SpawnToolkitTab(PropertiesTabId, FString(), EToolkitTabSpot::Details);
	}*/
}


/* ITextureEditorToolkit interface
 *****************************************************************************/

void FTextureEditorToolkit::CalculateTextureDimensions(int32& OutWidth, int32& OutHeight, int32& OutDepth, int32& OutArraySize, bool bInIncludeBorderSize) const
{
	OutWidth = static_cast<int32>(Texture->GetSurfaceWidth());
	OutHeight = static_cast<int32>(Texture->GetSurfaceHeight());
	OutDepth = static_cast<int32>(Texture->GetSurfaceDepth());
	OutArraySize = IsArrayTexture() ? (IsCubeTexture() ? Texture->GetSurfaceArraySize() / 6 : Texture->GetSurfaceArraySize()) : 0;
	const int32 BorderSize = GetDefault<UTextureEditorSettings>()->TextureBorderEnabled ? 1 : 0;

	
	if (UTexture2D* Texture2D = Cast<UTexture2D>(Texture))
	{
		// GetCPUCopyTexture waits on the build if there is one.
		if (Texture2D->Availability == ETextureAvailability::CPU &&
			Texture2D->IsAsyncCacheComplete())
		{
			if (UTexture2D* CpuTexture = Texture2D->GetCPUCopyTexture())
			{
				OutWidth = CpuTexture->GetSurfaceWidth();
				OutHeight = CpuTexture->GetSurfaceHeight();
				OutDepth = 1;
				OutArraySize = 0;
			}
		}
	}

	if (!PreviewEffectiveTextureWidth || !PreviewEffectiveTextureHeight)
	{
		OutWidth = 0;
		OutHeight = 0;
		OutDepth = 0;
		OutArraySize = 0;
		return;
	}

	// Fit is the same as fill, but doesn't scale up past 100%
	const ETextureEditorZoomMode CurrentZoomMode = GetZoomMode();
	if (CurrentZoomMode == ETextureEditorZoomMode::Fit || CurrentZoomMode == ETextureEditorZoomMode::Fill)
	{
		const int32 MaxWidth = FMath::Max(TextureViewport->GetViewport()->GetSizeXY().X - 2 * BorderSize, 0);
		const int32 MaxHeight = FMath::Max(TextureViewport->GetViewport()->GetSizeXY().Y - 2 * BorderSize, 0);

		if (IsVolumeTexture() && GetVolumeViewMode() == ETextureEditorVolumeViewMode::TextureEditorVolumeViewMode_VolumeTrace)
		{
			OutWidth = OutHeight = FMath::Min(MaxWidth, MaxHeight);
		}
		else if (MaxWidth * PreviewEffectiveTextureHeight < MaxHeight * PreviewEffectiveTextureWidth)
		{
			OutWidth = MaxWidth;
			OutHeight = FMath::DivideAndRoundNearest(OutWidth * PreviewEffectiveTextureHeight, PreviewEffectiveTextureWidth);
		}
		else
		{
			OutHeight = MaxHeight;
			OutWidth = FMath::DivideAndRoundNearest(OutHeight * PreviewEffectiveTextureWidth, PreviewEffectiveTextureHeight);
		}

		// If fit, then we only want to scale down
		// So if our natural dimensions are smaller than the viewport, we can just use those
		if (CurrentZoomMode == ETextureEditorZoomMode::Fit && (PreviewEffectiveTextureWidth < OutWidth || PreviewEffectiveTextureHeight < OutHeight))
		{
			OutWidth = PreviewEffectiveTextureWidth;
			OutHeight = PreviewEffectiveTextureHeight;
		}
	}
	else
	{
		OutWidth = static_cast<int32>(PreviewEffectiveTextureWidth * Zoom);
		OutHeight = static_cast<int32>(PreviewEffectiveTextureHeight * Zoom);
	}

	if (bInIncludeBorderSize)
	{
		OutWidth += 2 * BorderSize;
		OutHeight += 2 * BorderSize;
	}
}


ESimpleElementBlendMode FTextureEditorToolkit::GetColourChannelBlendMode( ) const
{
	if (Texture && (Texture->CompressionSettings == TC_Grayscale || Texture->CompressionSettings == TC_Alpha))
	{
		return SE_BLEND_Opaque;
	}

	// Add the red, green, blue, alpha and desaturation flags to the enum to identify the chosen filters
	uint32 Result = (uint32)SE_BLEND_RGBA_MASK_START;
	Result += bIsRedChannel ? (1 << 0) : 0;
	Result += bIsGreenChannel ? (1 << 1) : 0;
	Result += bIsBlueChannel ? (1 << 2) : 0;
	Result += bIsAlphaChannel ? (1 << 3) : 0;
	
	// If we only have one color channel active, enable color desaturation by default
	const int32 NumColorChannelsActive = (bIsRedChannel ? 1 : 0) + (bIsGreenChannel ? 1 : 0) + (bIsBlueChannel ? 1 : 0);
	const bool bIsDesaturationLocal = bIsDesaturation ? true : (NumColorChannelsActive==1);
	Result += bIsDesaturationLocal ? (1 << 4) : 0;

	return (ESimpleElementBlendMode)Result;
}

bool FTextureEditorToolkit::IsFitToViewport() const
{
	return IsCurrentZoomMode(ETextureEditorZoomMode::Fit);
}

bool FTextureEditorToolkit::IsFillToViewport() const
{
	return IsCurrentZoomMode(ETextureEditorZoomMode::Fill);
}

int32 FTextureEditorToolkit::GetMipLevel( ) const
{
	return GetUseSpecifiedMip() ? SpecifiedMipLevel : -1;
}

int32 FTextureEditorToolkit::GetLayer() const
{
	return SpecifiedLayer;
}

int32 FTextureEditorToolkit::GetSlice() const
{
	return GetUseSpecifiedSlice() ? HandleSliceEntryBoxValue().Get(-1) : -1;
}

int32 FTextureEditorToolkit::GetFace() const
{
	return GetUseSpecifiedFace() ? SpecifiedFace : -1;
}

UTexture* FTextureEditorToolkit::GetTexture( ) const
{
	return Texture;
}


bool FTextureEditorToolkit::HasValidTextureResource( ) const
{
	return Texture != nullptr && Texture->GetResource() != nullptr;
}


bool FTextureEditorToolkit::GetUseSpecifiedMip( ) const
{
	if (GetMaxMipLevel().Get(MIPLEVEL_MAX) > 0)
	{
		if (HandleMipLevelCheckBoxIsEnabled())
		{
			return bUseSpecifiedMipLevel;
		}

		// by default this is on
		return true; 
	}

	// disable the widgets if we have no mip maps
	return false;
}

bool FTextureEditorToolkit::GetUseSpecifiedSlice() const
{
	return HandleSliceCheckBoxIsEnabled() && bUseSpecifiedSlice;
}

bool FTextureEditorToolkit::GetUseSpecifiedFace() const
{
	return bUseSpecifiedFace;
}

double FTextureEditorToolkit::GetCustomZoomLevel( ) const
{
	return Zoom;
}


void FTextureEditorToolkit::PopulateQuickInfo( )
{
	check( Texture != nullptr );

	if (Texture->IsDefaultTexture())
	{
		ImportedText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_Imported_NA", "Imported: Computing..."));
		CurrentText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_Displayed_NA", "Displayed: Computing..."));
		MaxInGameText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_MaxInGame_NA", "Max In-Game: Computing..."));
		SizeText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_ResourceSize_NA", "Resource Size: Computing..."));
		MethodText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_Method_NA", "Method: Computing..."));
		LODBiasText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_LODBias_NA", "Combined LOD Bias: Computing..."));
		FormatText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_Format_NA", "Format: Computing..."));
		NumMipsText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_NumMips_NA", "Number of Mips: Computing..."));
		HasAlphaChannelText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_HasAlphaChannel_NA", "Format Supports Alpha: Computing..."));
		EncodeSpeedText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_EncodeSpeed_Computing", "Encode Speed: Computing..."));
		SceneCaptureSizeText->SetText(FText());
		SceneCaptureNameText->SetText(FText());
		return;
	}

	FTexturePlatformData** PlatformDataPtr = Texture->GetRunningPlatformData();
	if (PlatformDataPtr && PlatformDataPtr[0]) // Can be null if we haven't had a chance to call CachePlatformData on the texture (brand new)
	{
		FTexturePlatformData* PlatformData = PlatformDataPtr[0];
		FTexturePlatformData::FTextureEncodeResultMetadata const& ResultMetadata = PlatformData->ResultMetadata;
		if (ResultMetadata.bIsValid == false)
		{
			EncodeSpeedText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_EncodeSpeed_NA", "Encode Speed: N/A"));

			FText OodleInfoMissing = NSLOCTEXT("TextureEditor", "QuickInfo_Oodle_Missing", "<Metadata Missing>");
			OodleEncoderText->SetText(OodleInfoMissing);
			OodleEncodeSpeedText->SetText(OodleInfoMissing);
			OodleRDOText->SetText(OodleInfoMissing);
			OodleEffortText->SetText(OodleInfoMissing);
			OodleTilingText->SetText(OodleInfoMissing);
			OodleRDOSourceText->SetText(OodleInfoMissing);

			OodleRDOText->SetVisibility(EVisibility::Hidden);
			OodleEffortText->SetVisibility(EVisibility::Hidden);
			OodleTilingText->SetVisibility(EVisibility::Hidden);
			OodleRDOSourceText->SetVisibility(EVisibility::Hidden);

			OodleRDOEnabledLabel->SetVisibility(EVisibility::Hidden);
			OodleRDOSourceLabel->SetVisibility(EVisibility::Hidden);
			OodleEffortLabel->SetVisibility(EVisibility::Hidden);
			OodleTilingLabel->SetVisibility(EVisibility::Hidden);
		}
		else
		{
			//
			// Check if we need to compress new Oodle preview once we know we have
			// valid results.
			//
			bool AlreadyHaveResults = false;
			if (PlatformData->DerivedDataKey.GetIndex() == OodleCompressedPreviewDDCKey.GetIndex())
			{
				if (PlatformData->DerivedDataKey.IsType<FString>())
				{
					if (PlatformData->DerivedDataKey.Get<FString>() == OodleCompressedPreviewDDCKey.Get<FString>())
					{
						AlreadyHaveResults = true;
					}
				}
				else
				{
					if (*PlatformData->DerivedDataKey.Get<UE::DerivedData::FCacheKeyProxy>().AsCacheKey() == *OodleCompressedPreviewDDCKey.Get<UE::DerivedData::FCacheKeyProxy>().AsCacheKey())
					{
						AlreadyHaveResults = true;
					}
				}
			}

			if (AlreadyHaveResults == false)
			{
				if (bEstimateCompressionEnabled)
				{
					OutstandingEstimation = PlatformData->LaunchEstimateOnDiskSizeTask(OodleCompressor, OodleCompressionLevel, CompressionBlockSize, Texture->GetPathName());
				}

				// We only want to refresh the platform list when we have a settings change (AlreadyHaveResults, above)				
				{
					PlatformTextureFormatList.Empty();

					ITextureFormatManagerModule* TFMM = FModuleManager::GetModulePtr<ITextureFormatManagerModule>("TextureFormat");
					ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
					if (TPM)
					{
						for (FName PlatformName : AvailablePlatformNames)
						{
							ITargetPlatform* TargetPlatform = TPM->FindTargetPlatform(PlatformName);
							if (TargetPlatform)
							{
								TArray<TArray<FTextureBuildSettings>> Settings;
								Texture->GetTargetPlatformBuildSettings(TargetPlatform, Settings);

								if ( Settings.IsEmpty() || Settings[0].IsEmpty() )
								{
									continue;
								}

								FTextureBuildSettings& BuildSettings = Settings[0][0];
								FName TextureFormatName = UE::TextureBuildUtilities::TextureFormatRemovePlatformPrefixFromName(BuildSettings.TextureFormatName);
					
								ITextureFormat const* TextureFormat = TFMM->FindTextureFormat(BuildSettings.TextureFormatName);
								if (TextureFormat)
								{
									TSharedPtr<FPlatformTextureFormats> Formats = MakeShared<FPlatformTextureFormats>();
									Formats->PlatformName = PlatformName;
									Formats->TextureFormatName = TextureFormatName;
									Formats->PixelFormat_AlphaTrue = TextureFormat->GetEncodedPixelFormat(BuildSettings, true);
									Formats->PixelFormat_AlphaFalse = TextureFormat->GetEncodedPixelFormat(BuildSettings, false);
									Formats->bKnowAlphaTransparency = BuildSettings.bKnowAlphaTransparency;
									Formats->bHasTransparentAlpha = BuildSettings.bHasTransparentAlpha;

									// Technically this texture might not be VT; this means that the user wants it to, but there are
									// reasons why it might not.
									if (Texture->VirtualTextureStreaming)
									{
										// Our mip sizes don't really makes sense and the VT mip count isn't readily available, so we
										// don't report in these cases.
										Formats->bIsVirtual = true;
									}
									else
									{
										Formats->bIsVirtual = false;

										FTextureEngineParameters EngineParameters;
										FEncodedTextureDescription TextureDescription;
										FEncodedTextureExtendedData TextureExtendedData;
										bool bHaveEncodedTextureDescription = BuildSettings.GetEncodedTextureDescriptionFromSourceMips(
											&TextureDescription, TextureFormat, 
											Texture->Source.GetSizeX(), Texture->Source.GetSizeY(), Texture->Source.GetNumSlices(),
											Texture->Source.GetNumMips(), Formats->bHasTransparentAlpha);

										if (bHaveEncodedTextureDescription)
										{
											if (BuildSettings.TilerEvenIfNotSharedLinear)
											{
												TextureExtendedData = BuildSettings.TilerEvenIfNotSharedLinear->GetExtendedDataForTexture(TextureDescription, BuildSettings.LODBias);
											}
											else
											{
												TextureDescription.GetLinearMipSizesInBytes(TextureExtendedData.MipSizesInBytes);
											}

											const UTextureLODSettings* LODSettings = 
												UDeviceProfileManager::Get().FindProfile(TargetPlatform->CookingDeviceProfileName())->GetTextureLODSettings();
											int32 FirstMipToSerialize = LODSettings->CalculateLODBias(
												TextureDescription.TopMipSizeX, TextureDescription.TopMipSizeY, Texture->GetMinTextureSize(), Texture->GetMaxTextureSize(),
												Texture->LODGroup, Texture->LODBias, 0, Texture->MipGenSettings, false, Texture->IsStreamable());

											int32 StrippedNumMips = TextureDescription.NumMips;
											const int32 LastMip = FMath::Max(StrippedNumMips - 1, 0);
											const int32 FirstMipTailMip = StrippedNumMips - TextureExtendedData.NumMipsInTail;
											check(FirstMipTailMip >= 0);
											FirstMipToSerialize = FMath::Clamp(FirstMipToSerialize, 0, TextureExtendedData.NumMipsInTail > 0 ? FirstMipTailMip : LastMip);
											StrippedNumMips = FMath::Max(0, StrippedNumMips - FirstMipToSerialize);
											

											int32 NumNonStreamingMips = TextureDescription.GetNumNonStreamingMips(&TextureExtendedData, EngineParameters);
											NumNonStreamingMips = FMath::Min(NumNonStreamingMips, StrippedNumMips);
											int32 FirstInlineMip = StrippedNumMips - NumNonStreamingMips;
											int32 OptionalMipCount = 0;

											if (StrippedNumMips > 0 )
											{
												const int32 FirstMipWidth  = TextureDescription.GetMipWidth(FirstMipToSerialize);
												const int32 FirstMipHeight = TextureDescription.GetMipHeight(FirstMipToSerialize);

												OptionalMipCount = LODSettings->CalculateNumOptionalMips(Texture->LODGroup, FirstMipWidth, FirstMipHeight, StrippedNumMips, FirstInlineMip, Texture->MipGenSettings);

												// OptionalMips must be streaming mips.
												check(OptionalMipCount <= FirstInlineMip);
											}

											Formats->LODBias = FirstMipToSerialize;
											Formats->NumMips_TotalAfterLOD = TextureDescription.NumMips - FirstMipToSerialize;
											Formats->Width = TextureDescription.GetMipWidth(FirstMipToSerialize);
											Formats->Height = TextureDescription.GetMipHeight(FirstMipToSerialize);
											Formats->NumMips_Tail = TextureExtendedData.NumMipsInTail;
											
											// We need to make sure all these indices have been adjusted for LODBias.
											const int32 OptionalMipsEndAt = Formats->LODBias + OptionalMipCount;
											const int32 InlineMipsStartAt = Formats->LODBias + FirstInlineMip;
											const int32 NumAddressableMips = TextureDescription.NumMips - ((TextureExtendedData.NumMipsInTail > 0) ? (TextureExtendedData.NumMipsInTail - 1) : 0);
										
											int32 MipIndex = Formats->LODBias;
											for (; MipIndex < OptionalMipsEndAt; MipIndex++)
											{
												Formats->MipChainSizeInBytes_Optional += TextureExtendedData.MipSizesInBytes[MipIndex];
												Formats->NumMips_Optional++;												
											}

											for (; MipIndex < InlineMipsStartAt; MipIndex++)
											{
												Formats->NumMips_Streaming++;
												Formats->MipChainSizeInBytes_Streaming += TextureExtendedData.MipSizesInBytes[MipIndex];
											}
											
											for (; MipIndex < NumAddressableMips; MipIndex++)
											{
												Formats->MipChainSizeInBytes_Inline += TextureExtendedData.MipSizesInBytes[MipIndex];
											}

											Formats->MipSizesInBytes = MoveTemp(TextureExtendedData.MipSizesInBytes);
										}
									}

									PlatformTextureFormatList.Add(Formats);
								}
							}
						}
					}

					PlatformTextureFormatsWidget->RebuildList();
				}


				OodleCompressedPreviewDDCKey = PlatformDataPtr[0]->DerivedDataKey;
			}

			// If we have an outstanding estimation task, update UI when complete.
			if (OutstandingEstimation.IsValid())
			{
				if (OutstandingEstimation.IsReady())
				{
					TTuple<uint64, uint64> Result = OutstandingEstimation.Get();

					OodleEstimateRaw->SetText(FText::AsMemory(Result.Get<1>()));
					OodleEstimateCompressed->SetText(FText::AsMemory(Result.Get<0>()));

					OutstandingEstimation = TFuture<TTuple<uint64, uint64>>();
				}
				else
				{
					OodleEstimateRaw->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_Oodle_Working", "Working..."));
					OodleEstimateCompressed->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_Oodle_Working", "Working..."));
				}
			}

			OodleEncoderText->SetText(FText::FromName(ResultMetadata.Encoder));

			if (ResultMetadata.bSupportsEncodeSpeed == false)
			{
				EncodeSpeedText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_EncodeSpeed_Unsup", "Encode Speed: Unsupported"));
				OodleEncodeSpeedText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_Oodle_SpeedUnsup", "Unsupported"));

				FText OodleInfoNA = NSLOCTEXT("TextureEditor", "QuickInfo_Oodle_NA", "N/A");
				OodleRDOText->SetText(OodleInfoNA);
				OodleEffortText->SetText(OodleInfoNA);
				OodleTilingText->SetText(OodleInfoNA);
				OodleRDOSourceText->SetText(OodleInfoNA);

				OodleRDOText->SetVisibility(EVisibility::Hidden);
				OodleEffortText->SetVisibility(EVisibility::Hidden);
				OodleTilingText->SetVisibility(EVisibility::Hidden);
				OodleRDOSourceText->SetVisibility(EVisibility::Hidden);

				OodleRDOEnabledLabel->SetVisibility(EVisibility::Hidden);
				OodleRDOSourceLabel->SetVisibility(EVisibility::Hidden);
				OodleEffortLabel->SetVisibility(EVisibility::Hidden);
				OodleTilingLabel->SetVisibility(EVisibility::Hidden);

			}
			else
			{
				OodleRDOText->SetVisibility(EVisibility::Visible);
				OodleEffortText->SetVisibility(EVisibility::Visible);
				OodleTilingText->SetVisibility(EVisibility::Visible);
				OodleRDOSourceText->SetVisibility(EVisibility::Visible);

				OodleRDOEnabledLabel->SetVisibility(EVisibility::Visible);
				OodleRDOSourceLabel->SetVisibility(EVisibility::Visible);
				OodleEffortLabel->SetVisibility(EVisibility::Visible);
				OodleTilingLabel->SetVisibility(EVisibility::Visible);

				if (ResultMetadata.bWasEditorCustomEncoding)
				{
					EncodeSpeedText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_EncodeSpeed_Custom", "Encode Speed: Custom"));
					OodleEncodeSpeedText->SetText(NSLOCTEXT("TextureEditor", "QuickInfoDetails_EncodeSpeed_Custom", "Custom"));
				}
				else
				{
					EncodeSpeedText->SetText(
						ResultMetadata.EncodeSpeed == (uint8)ETextureEncodeSpeed::Fast ?
							NSLOCTEXT("TextureEditor", "QuickInfo_EncodeSpeed_Fast", "Encode Speed: Fast") : 
							NSLOCTEXT("TextureEditor", "QuickInfo_EncodeSpeed_Final", "Encode Speed: Final")
					);
					OodleEncodeSpeedText->SetText(
						ResultMetadata.EncodeSpeed == (uint8)ETextureEncodeSpeed::Fast ?
						NSLOCTEXT("TextureEditor", "QuickInfoDetails_EncodeSpeed_Fast", "Fast") :
						NSLOCTEXT("TextureEditor", "QuickInfoDetails_EncodeSpeed_Final", "Final")
					);
				}

				if (ResultMetadata.OodleRDO == 0)
				{
					const FResolvedTextureEncodingSettings& Settings = FResolvedTextureEncodingSettings::Get();
					const bool bDisabledGlobally = ResultMetadata.EncodeSpeed == (uint8)ETextureEncodeSpeed::Fast ? !Settings.Project.bFastUsesRDO : !Settings.Project.bFinalUsesRDO;

					OodleRDOText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_Oodle_RDODisable", "Disabled"));
					if (ResultMetadata.bWasEditorCustomEncoding)
					{
						OodleRDOSourceText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_Oodle_RDOSource_Custom", "Custom"));
					}
					else if (bDisabledGlobally)
					{
						OodleRDOSourceText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_Oodle_RDOSourceDisableSettings", "Disabled By Project Settings"));
					}
					else
					{
						if (ResultMetadata.RDOSource == FTexturePlatformData::FTextureEncodeResultMetadata::OodleRDOSource::Default)
						{
							OodleRDOSourceText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_Oodle_RDOSourceDisableLCA_Default", "Disabled By Project (Lossy Compression Amount)"));
						}
						else if (ResultMetadata.RDOSource == FTexturePlatformData::FTextureEncodeResultMetadata::OodleRDOSource::Texture)
						{
							OodleRDOSourceText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_Oodle_RDOSourceDisableLCA_Texture", "Disabled By Texture (Lossy Compression Amount)"));
						}
						else if (ResultMetadata.RDOSource == FTexturePlatformData::FTextureEncodeResultMetadata::OodleRDOSource::LODGroup)
						{
							OodleRDOSourceText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_Oodle_RDOSourceDisableLCA_LODGroup", "Disabled By LODGroup (Lossy Compression Amount)"));
						}
					}
				}
				else
				{
					OodleRDOText->SetText(FText::AsNumber(ResultMetadata.OodleRDO));

					if (ResultMetadata.bWasEditorCustomEncoding)
					{
						OodleRDOSourceText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_Oodle_RDOSource_Custom", "Custom"));
					}
					else if (ResultMetadata.RDOSource == FTexturePlatformData::FTextureEncodeResultMetadata::OodleRDOSource::Default)
					{
						OodleRDOSourceText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_Oodle_RDOSource_Default", "Project (Lambda)"));
					}
					else if (ResultMetadata.RDOSource == FTexturePlatformData::FTextureEncodeResultMetadata::OodleRDOSource::Texture)
					{
						OodleRDOSourceText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_Oodle_RDOSource_Texture", "Texture (Lossy Compression Amount)"));
					}
					else if (ResultMetadata.RDOSource == FTexturePlatformData::FTextureEncodeResultMetadata::OodleRDOSource::LODGroup)
					{
						OodleRDOSourceText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_Oodle_RDOSource_LODGroup", "LODGroup (Lossy Compression Amount)"));
					}
				}

				UEnum* EncodeEffortEnum = StaticEnum<ETextureEncodeEffort>();
				OodleEffortText->SetText(FText::AsCultureInvariant(EncodeEffortEnum->GetNameStringByValue(ResultMetadata.OodleEncodeEffort)));

				UEnum* UniversalTilingEnum = StaticEnum<ETextureUniversalTiling>();
				OodleTilingText->SetText(FText::AsCultureInvariant(UniversalTilingEnum->GetNameStringByValue(ResultMetadata.OodleUniversalTiling)));
			} // end if encode speed supported
		} // end if results metadata valid

		if (Texture->Source.IsValid() &&
			Texture->Source.HasLayerColorInfo())
		{
			// Make a 1x1 image with our min color to use for alpha detection.
			
			TArray<FTextureSourceLayerColorInfo> LayerColorInfo;
			Texture->Source.GetLayerColorInfo(LayerColorInfo);

			FImageView View(&LayerColorInfo[0].ColorMin, 1, 1);

			bool bSourceAlphaDetected = FImageCore::DetectAlphaChannel(View);
			SourceMipsAlphaDetectedText->SetText(FText::Format(NSLOCTEXT("TextureEditor", "QuickInfo_SourceAlphaDetected", "Source Alpha Detected: {0}"),
				bSourceAlphaDetected ? NSLOCTEXT("TextureEditor", "True", "True") : NSLOCTEXT("TextureEditor", "False", "False")));
			DetectSourceAlphaButton->SetVisibility(EVisibility::Collapsed);
		}
		else
		{
			SourceMipsAlphaDetectedText->SetText(FText::Format(NSLOCTEXT("TextureEditor", "QuickInfo_SourceAlphaDetected", "Source Alpha Detected: {0}"),
				NSLOCTEXT("TextureEditor", "Unknown", "Unknown")));
			DetectSourceAlphaButton->SetVisibility(Texture->Source.IsValid() ? EVisibility::Visible : EVisibility::Collapsed);
		}
	} // end if valid platform data

	UTexture2D* Texture2D = Cast<UTexture2D>(Texture);


	const bool bIsVolume = IsVolumeTexture();
	const bool bIsArray = IsArrayTexture();
	const bool bIsCube = IsCubeTexture();

	const int32 SurfaceWidth = static_cast<int32>(Texture->GetSurfaceWidth());
	const int32 SurfaceHeight = static_cast<int32>(Texture->GetSurfaceHeight());
	const int32 SurfaceDepth = static_cast<int32>(Texture->GetSurfaceDepth());
	const int32 NumSurfaces = Texture->GetSurfaceArraySize();
	const int32 ArraySize = bIsArray ? (bIsCube ? NumSurfaces / 6 : NumSurfaces) : 0;

	// this is the size of just one block :
	//	use GetLogicalSize for all blocks
	const int32 ImportedWidth = Texture->Source.IsValid() ? Texture->Source.GetSizeX() : SurfaceWidth;
	const int32 ImportedHeight = Texture->Source.IsValid() ? Texture->Source.GetSizeY() : SurfaceHeight;
	const int32 ImportedDepth = Texture->Source.IsValid() ? (bIsVolume ? Texture->Source.GetNumSlices() : 0) : SurfaceDepth;

	const int32 ImportedNumBlocks = Texture->Source.GetNumBlocks();
	const FIntPoint ImportedSizeInBlocks = Texture->Source.GetSizeInBlocks();

	const FStreamableRenderResourceState SRRState = Texture->GetStreamableResourceState();
	const int32 ActualMipBias = SRRState.IsValid() ? (SRRState.ResidentFirstLODIdx() + SRRState.AssetLODBias) : Texture->GetCachedLODBias();
	// Editor dimensions (takes user specified mip setting into account)
	const int32 MipLevel = ActualMipBias + FMath::Max(GetMipLevel(), 0);
	PreviewEffectiveTextureWidth = SurfaceWidth ? FMath::Max(SurfaceWidth >> MipLevel, 1) : 0;
	PreviewEffectiveTextureHeight = SurfaceHeight ? FMath::Max(SurfaceHeight >> MipLevel, 1) : 0;
	const int32 PreviewEffectiveTextureDepth = SurfaceDepth ? FMath::Max(SurfaceDepth >> MipLevel, 1) : 0;

	// In game max bias and dimensions
	const int32 MaxResMipBias = Texture2D ? (Texture2D->GetNumMips() - Texture2D->GetNumMipsAllowed(true)) : Texture->GetCachedLODBias();
	const int32 MaxInGameWidth = SurfaceWidth ? FMath::Max(SurfaceWidth >> MaxResMipBias, 1) : 0;
	const int32 MaxInGameHeight = SurfaceHeight ? FMath::Max(SurfaceHeight >> MaxResMipBias, 1) : 0;
	const int32 MaxInGameDepth = SurfaceDepth ? FMath::Max(SurfaceDepth >> MaxResMipBias, 1) : 0;

	// Texture asset size
	const int64 ResourceSize = PlatformDataPtr && *PlatformDataPtr ? (*PlatformDataPtr)->GetPayloadSize(ActualMipBias) : Texture->GetResourceSizeBytes(EResourceSizeMode::Exclusive);

	FText ImportedCubemapInfo;
	FText DisplayedCubemapInfo;
	FText InGameCubemapInfo;
	if (bIsCube)
	{
		InGameCubemapInfo = NSLOCTEXT("TextureEditor", "QuickInfo_PerCubeSide", "*6 (Cubemap)");
		ImportedCubemapInfo = Texture->Source.IsLongLatCubemap() ? NSLOCTEXT("TextureEditor", "QuickInfo_LongLat", " (LongLat)") : InGameCubemapInfo;
		if (GetFace() >= 0)
		{
			DisplayedCubemapInfo = NSLOCTEXT("TextureEditor", "QuickInfo_CubemapFace", " (Cubemap face)");
		}
		else if (GetCubemapViewMode() == TextureEditorCubemapViewMode_2DView)
		{
			// when using 2D view mode and face index is not specified, cubemaps are previewed as longlat unwraps, which have double width of a cube face
			PreviewEffectiveTextureWidth *= 2;
			DisplayedCubemapInfo = NSLOCTEXT("TextureEditor", "QuickInfo_LongLat", " (LongLat)");
		}
		else 
		{
			DisplayedCubemapInfo = InGameCubemapInfo;
		}
	}

	FNumberFormattingOptions FormatOptions;
	FormatOptions.UseGrouping = false;

	if (bIsVolume)
	{
		ImportedText->SetText(FText::Format( NSLOCTEXT("TextureEditor", "QuickInfo_Imported_3x", "Imported: {0}x{1}x{2}"), FText::AsNumber(ImportedWidth, &FormatOptions), FText::AsNumber(ImportedHeight, &FormatOptions), FText::AsNumber(ImportedDepth, &FormatOptions)));
		CurrentText->SetText(FText::Format( NSLOCTEXT("TextureEditor", "QuickInfo_Displayed_3x", "Displayed: {0}x{1}x{2}"), FText::AsNumber(PreviewEffectiveTextureWidth, &FormatOptions ), FText::AsNumber(PreviewEffectiveTextureHeight, &FormatOptions), FText::AsNumber(PreviewEffectiveTextureDepth, &FormatOptions)));
		MaxInGameText->SetText(FText::Format( NSLOCTEXT("TextureEditor", "QuickInfo_MaxInGame_3x_v1", "Max In-Game: {0}x{1}x{2}"), FText::AsNumber(MaxInGameWidth, &FormatOptions), FText::AsNumber(MaxInGameHeight, &FormatOptions), FText::AsNumber(MaxInGameDepth, &FormatOptions)));

		if (GetVolumeViewMode() == ETextureEditorVolumeViewMode::TextureEditorVolumeViewMode_VolumeTrace)
		{
			PreviewEffectiveTextureWidth = PreviewEffectiveTextureHeight = FMath::Max(PreviewEffectiveTextureWidth, PreviewEffectiveTextureHeight);
		}
		else
		{
			int32 NumTilesX = 0;
			int32 NumTilesY = 0;
			GetBestFitForNumberOfTiles(PreviewEffectiveTextureDepth, NumTilesX, NumTilesY);
			PreviewEffectiveTextureWidth *= NumTilesX;
			PreviewEffectiveTextureHeight *= NumTilesY;
		}
	}
	else if (bIsArray)
	{
		ImportedText->SetText(FText::Format(NSLOCTEXT("TextureEditor", "QuickInfo_Imported_3x_v2", "Imported: {0}x{1}{2}*{3}"), FText::AsNumber(ImportedWidth, &FormatOptions), FText::AsNumber(ImportedHeight, &FormatOptions), ImportedCubemapInfo, FText::AsNumber(ArraySize, &FormatOptions)));
		CurrentText->SetText(FText::Format(NSLOCTEXT("TextureEditor", "QuickInfo_Displayed_3x_v2", "Displayed: {0}x{1}{2}*{3}"), FText::AsNumber(PreviewEffectiveTextureWidth, &FormatOptions), FText::AsNumber(PreviewEffectiveTextureHeight, &FormatOptions), DisplayedCubemapInfo, FText::AsNumber(ArraySize, &FormatOptions)));
		MaxInGameText->SetText(FText::Format(NSLOCTEXT("TextureEditor", "QuickInfo_MaxInGame_3x_v2", "Max In-Game: {0}x{1}{2}*{3}"), FText::AsNumber(MaxInGameWidth, &FormatOptions), FText::AsNumber(MaxInGameHeight, &FormatOptions), InGameCubemapInfo, FText::AsNumber(ArraySize, &FormatOptions)));
	}
	else
	{
		if ( ImportedNumBlocks > 1 )
		{
		    ImportedText->SetText(FText::Format( NSLOCTEXT("TextureEditor", "QuickInfo_Imported_Blocks", "Imported: {0}x{1}x{2}x{3}"), 
				FText::AsNumber(ImportedWidth, &FormatOptions), FText::AsNumber(ImportedHeight, &FormatOptions), 
				FText::AsNumber(ImportedSizeInBlocks.X, &FormatOptions), FText::AsNumber(ImportedSizeInBlocks.Y, &FormatOptions)));
		}
		else
		{
		    ImportedText->SetText(FText::Format( NSLOCTEXT("TextureEditor", "QuickInfo_Imported_2x", "Imported: {0}x{1}{2}"), FText::AsNumber(ImportedWidth, &FormatOptions), FText::AsNumber(ImportedHeight, &FormatOptions), ImportedCubemapInfo));
		}

		CurrentText->SetText(FText::Format( NSLOCTEXT("TextureEditor", "QuickInfo_Displayed_2x", "Displayed: {0}x{1}{2}"), FText::AsNumber(PreviewEffectiveTextureWidth, &FormatOptions ), FText::AsNumber(PreviewEffectiveTextureHeight, &FormatOptions), DisplayedCubemapInfo));
		MaxInGameText->SetText(FText::Format( NSLOCTEXT("TextureEditor", "QuickInfo_MaxInGame_2x", "Max In-Game: {0}x{1}{2}"), FText::AsNumber(MaxInGameWidth, &FormatOptions), FText::AsNumber(MaxInGameHeight, &FormatOptions), InGameCubemapInfo));
	}

	// GetCPUCopyTexture waits on the build if there is one.
	if (Texture2D &&
		Texture2D->Availability == ETextureAvailability::CPU &&
		Texture2D->IsAsyncCacheComplete())
	{
		if (UTexture2D* CpuTexture = Texture2D->GetCPUCopyTexture(); CpuTexture)
		{
			PreviewEffectiveTextureWidth = CpuTexture->GetSurfaceWidth();
			PreviewEffectiveTextureHeight = CpuTexture->GetSurfaceHeight();
		}
	}

	if (ViewingPlatform != NAME_None)
	{
		// Right now the resource size is the size of the texture we are viewing which could be decoded BGRA8 and not
		// what you expect. This has been causing confusion so until I do UE-212930 we just make it clear we don't know.
		SizeText->SetText(NSLOCTEXT("TextureEditor", "QuickInfo_ResourceSizeUnknown", "Resource Size: <unknown due to platform preview>"));
	}
	else
	{
		SizeText->SetText(FText::Format(NSLOCTEXT("TextureEditor", "QuickInfo_ResourceSize", "Resource Size: {0} KB"), FText::AsNumber(FMath::DivideAndRoundNearest(ResourceSize, (int64)1024), &FormatOptions)));
	}

	ETextureStreamingMethod StreamingMethod = Texture->GetTextureStreamingMethod();
	FText Method;
	switch(StreamingMethod)
	{
		default:
		case TSM_Invalid:
			Method = NSLOCTEXT("TextureEditor", "QuickInfo_MethodInvalidStreamed", "Invalid");
			break;
		case TSM_NotStreamed:
			Method = NSLOCTEXT("TextureEditor", "QuickInfo_MethodNotStreamed", "Not Streamed");
			break;
		case TSM_Streamed:
			Method = NSLOCTEXT("TextureEditor", "QuickInfo_MethodStreamed", "Streamed");
			break;
		case TSM_VirtualStreamed:
			Method = NSLOCTEXT("TextureEditor", "QuickInfo_MethodVirtualStreamed", "Virtual Streamed");
			break;
	}

	MethodText->SetText(FText::Format(NSLOCTEXT("TextureEditor", "QuickInfo_Method", "Method: {0}"), Method));
	LODBiasText->SetText(FText::Format(NSLOCTEXT("TextureEditor", "QuickInfo_LODBias", "Combined LOD Bias: {0}"), FText::AsNumber(Texture->GetCachedLODBias())));

	EPixelFormat TextureFormat = GetPixelFormat();
	if (TextureFormat != PF_MAX)
	{
		if (ViewingPlatform != NAME_None)
		{
			// This can end up unknown for several reasons (can't determine alpha primarily) but should usually have it.
			EPixelFormat EncodedPixelFormat = PF_Unknown;
			if (PlatformDataPtr && 
				PlatformDataPtr[0] && // Can be null if we haven't had a chance to call CachePlatformData on the texture (brand new)
				PlatformDataPtr[0]->ResultMetadata.bIsValid)
			{
				EncodedPixelFormat = PlatformDataPtr[0]->ResultMetadata.EncodedFormat;
			}

			FormatText->SetText(FText::Format(NSLOCTEXT("TextureEditor", "QuickInfo_PlatformFormat", "Format: Viewing {0} Actual {1}"), 
				FText::FromString(GPixelFormats[(uint8)TextureFormat].Name),
				FText::FromString(GPixelFormats[(uint8)EncodedPixelFormat].Name)));
		}
		else
		{
			FormatText->SetText(FText::Format(NSLOCTEXT("TextureEditor", "QuickInfo_Format", "Format: {0}"), FText::FromString(GPixelFormats[(uint8)TextureFormat].Name)));
		}
	}

	// This "Has Alpha Channel" is whether the GPU format can represent alpha in the format (eg. is it DXT1 vs DXT5)
	//	it does not tell you if the texture actually has non-opaque alpha
	EPixelFormatChannelFlags ValidTextureChannels = GetPixelFormatValidChannels(TextureFormat);
	HasAlphaChannelText->SetText(FText::Format(NSLOCTEXT("TextureEditor", "QuickInfo_HasAlphaChannel", "Format Supports Alpha: {0}"),
		EnumHasAnyFlags(ValidTextureChannels, EPixelFormatChannelFlags::A) ? NSLOCTEXT("TextureEditor", "True", "True") : NSLOCTEXT("TextureEditor", "False", "False")));
	HasAlphaChannelText->SetVisibility((ValidTextureChannels != EPixelFormatChannelFlags::None) ? EVisibility::Visible : EVisibility::Collapsed);

	int32 NumMips = GetNumMips();
	NumMipsText->SetText(FText::Format(NSLOCTEXT("TextureEditor", "QuickInfo_NumMips", "Number of Mips: {0}"), FText::AsNumber(NumMips)));

	uint64 CaptureSize = 0;
	FName CaptureName;
	if (Texture->GetTextureClass() == ETextureClass::RenderTarget)
	{
		for (TObjectIterator<USceneCaptureComponent2D> It; It; ++It)
		{
			USceneCaptureComponent2D* SceneCaptureComponent = *It;
			check( SceneCaptureComponent != nullptr );
			
			if (SceneCaptureComponent->TextureTarget == Texture )
			{
				TRefCountPtr<FSceneCaptureMemorySize> CaptureMemorySize = SceneCaptureComponent->CaptureMemorySize;
				if ( CaptureMemorySize.IsValid())
				{
					// Could have multiple scene captures blending to the same render target -- add them all up, but only show the name of the last one...
					CaptureSize += CaptureMemorySize->Size;
					CaptureName = SceneCaptureComponent->GetOwner() ? SceneCaptureComponent->GetOwner()->GetFName() : FName();
				}
			}
		}
	}
	if (CaptureSize)
	{
		SceneCaptureSizeText->SetText(FText::Format(NSLOCTEXT("TextureEditor", "QuickInfo_SceneCaptureSize", "Scene Capture: {0} KB"), FText::AsNumber(FMath::DivideAndRoundNearest(CaptureSize, (uint64)1024), &FormatOptions)));
		SceneCaptureNameText->SetText(FText::Format(FText::FromString(TEXT("({0})")), FText::FromName(CaptureName)));
	}
	else
	{
		SceneCaptureSizeText->SetText(FText());
		SceneCaptureNameText->SetText(FText());
	}
}


void FTextureEditorToolkit::SetZoomMode( const ETextureEditorZoomMode InZoomMode )
{
	// Update our own zoom mode
	ZoomMode = InZoomMode;
	
	// And also save it so it's used for new texture editors
	UTextureEditorSettings& Settings = *GetMutableDefault<UTextureEditorSettings>();
	Settings.ZoomMode = ZoomMode;
	Settings.PostEditChange();
}

ETextureEditorZoomMode FTextureEditorToolkit::GetZoomMode() const
{
	// Each texture editors keeps a local zoom mode so that it can be changed without affecting other open editors
	return ZoomMode;
}

double FTextureEditorToolkit::CalculateDisplayedZoomLevel() const
{
	// Avoid calculating dimensions if we're custom anyway
	if (GetZoomMode() == ETextureEditorZoomMode::Custom)
	{
		return Zoom;
	}

	int32 DisplayWidth, DisplayHeight, DisplayDepth, DisplayArraySize;
	CalculateTextureDimensions(DisplayWidth, DisplayHeight, DisplayDepth, DisplayArraySize, false);
	if (PreviewEffectiveTextureHeight != 0)
	{
		return (double)DisplayHeight / PreviewEffectiveTextureHeight;
	}
	else if (PreviewEffectiveTextureWidth != 0)
	{
		return (double)DisplayWidth / PreviewEffectiveTextureWidth;
	}
	else
	{
		return 0;
	}
}

void FTextureEditorToolkit::SetCustomZoomLevel( double ZoomValue )
{
	// snap to discrete steps so that if we are nearly at 1.0 or 2.0, we hit them exactly:
	//ZoomValue = FMath::GridSnap(ZoomValue, MinZoom/4.0);

	double LogZoom = log2(ZoomValue);
	// the mouse wheel zoom is quantized on ZoomFactorLogSteps
	//	but that's too chunky for the drag slider, give it more steps, but on the same quantization grid
	double QuantizationSteps = ZoomFactorLogSteps*2.0;
	double LogZoomQuantized = (1.0/QuantizationSteps) * (double)FMath::RoundToInt( QuantizationSteps * LogZoom );
	ZoomValue = pow(2.0,LogZoomQuantized);

	ZoomValue = FMath::Clamp(ZoomValue, MinZoom, MaxZoom);
	
	// set member variable "Zoom"
	Zoom = ZoomValue;

	// For now we also want to be in custom mode whenever this is changed
	SetZoomMode(ETextureEditorZoomMode::Custom);
}


void FTextureEditorToolkit::ZoomIn( )
{
	// mouse wheel zoom
	const double CurrentZoom = CalculateDisplayedZoomLevel();
	SetCustomZoomLevel(CurrentZoom * ZoomFactor);
}


void FTextureEditorToolkit::ZoomOut( )
{
	const double CurrentZoom = CalculateDisplayedZoomLevel();
	SetCustomZoomLevel(CurrentZoom / ZoomFactor);
}

float FTextureEditorToolkit::GetVolumeOpacity() const
{
	return VolumeOpacity;
}

void FTextureEditorToolkit::SetVolumeOpacity(float InVolumeOpacity)
{
	VolumeOpacity = FMath::Clamp(InVolumeOpacity, 0.f, 1.f);
}

ETextureEditorVolumeViewMode FTextureEditorToolkit::GetVolumeViewMode() const
{
	// Each texture editor keeps a local volume view mode so that it can be changed without affecting other open editors
	return VolumeViewMode;
}

void FTextureEditorToolkit::SetVolumeViewMode(const ETextureEditorVolumeViewMode InVolumeViewMode)
{
	// Update our own volume view mode
	VolumeViewMode = InVolumeViewMode;

	// And also save it so it's used for new texture editors
	UTextureEditorSettings& Settings = *GetMutableDefault<UTextureEditorSettings>();
	Settings.VolumeViewMode = VolumeViewMode;
	Settings.PostEditChange();
}

ETextureEditorCubemapViewMode FTextureEditorToolkit::GetCubemapViewMode() const
{
	// Each texture editor keeps a local cubemap view mode so that it can be changed without affecting other open editors
	return CubemapViewMode;
}

void FTextureEditorToolkit::SetCubemapViewMode(const ETextureEditorCubemapViewMode InCubemapViewMode)
{
	// Update our own cubemap view mode
	CubemapViewMode = InCubemapViewMode;

	// And also save it so it's used for new texture editors
	UTextureEditorSettings& Settings = *GetMutableDefault<UTextureEditorSettings>();
	Settings.CubemapViewMode = CubemapViewMode;
	Settings.PostEditChange();
}

bool FTextureEditorToolkit::IsUsingOrientation() const
{
	return (IsVolumeTexture() && GetVolumeViewMode() == TextureEditorVolumeViewMode_VolumeTrace) ||
		(IsCubeTexture() && GetCubemapViewMode() == TextureEditorCubemapViewMode_3DView && GetFace() < 0);
}

const FRotator& FTextureEditorToolkit::GetOrientation() const
{
	return Orientation;
}

void FTextureEditorToolkit::SetOrientation(const FRotator& InOrientation)
{
	Orientation = InOrientation;
}

void FTextureEditorToolkit::ResetOrientation()
{
	SetOrientation(IsVolumeTexture() ? FRotator(90, 0, -90) : FRotator(0, 0, 0));
}

ETextureEditorSampling FTextureEditorToolkit::GetSampling() const
{
	return Sampling;
}

/* IToolkit interface
 *****************************************************************************/

FText FTextureEditorToolkit::GetBaseToolkitName( ) const
{
	return LOCTEXT("AppLabel", "Texture Editor");
}


FName FTextureEditorToolkit::GetToolkitFName( ) const
{
	return FName("TextureEditor");
}


FLinearColor FTextureEditorToolkit::GetWorldCentricTabColorScale( ) const
{
	return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
}


FString FTextureEditorToolkit::GetWorldCentricTabPrefix( ) const
{
	return LOCTEXT("WorldCentricTabPrefix", "Texture ").ToString();
}


/* FGCObject interface
 *****************************************************************************/

void FTextureEditorToolkit::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject(Texture);
	TextureViewport->AddReferencedObjects(Collector);
}


/* FEditorUndoClient interface
 *****************************************************************************/

void FTextureEditorToolkit::PostUndo( bool bSuccess )
{
}


void FTextureEditorToolkit::PostRedo( bool bSuccess )
{
	PostUndo(bSuccess);
}


/* FTextureEditorToolkit implementation
 *****************************************************************************/

void FTextureEditorToolkit::BindCommands( )
{
	const FTextureEditorCommands& Commands = FTextureEditorCommands::Get();

	ToolkitCommands->MapAction(
		Commands.RedChannel,
		FExecuteAction::CreateSP(this, &FTextureEditorToolkit::OnChannelButtonCheckStateChanged, ETextureChannelButton::Red),
		FCanExecuteAction());

	ToolkitCommands->MapAction(
		Commands.GreenChannel,
		FExecuteAction::CreateSP(this, &FTextureEditorToolkit::OnChannelButtonCheckStateChanged, ETextureChannelButton::Green),
		FCanExecuteAction());

	ToolkitCommands->MapAction(
		Commands.BlueChannel,
		FExecuteAction::CreateSP(this, &FTextureEditorToolkit::OnChannelButtonCheckStateChanged, ETextureChannelButton::Blue),
		FCanExecuteAction());

	ToolkitCommands->MapAction(
		Commands.AlphaChannel,
		FExecuteAction::CreateSP(this, &FTextureEditorToolkit::OnChannelButtonCheckStateChanged, ETextureChannelButton::Alpha),
		FCanExecuteAction());

	ToolkitCommands->MapAction(
		Commands.Desaturation,
		FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleDesaturationChannelActionExecute),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FTextureEditorToolkit::HandleDesaturationChannelActionIsChecked));

	ToolkitCommands->MapAction(
		Commands.FillToViewport,
		FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleFillToViewportActionExecute));

	ToolkitCommands->MapAction(
		Commands.FitToViewport,
		FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleFitToViewportActionExecute));

	ToolkitCommands->MapAction(
		Commands.ZoomToNatural,
		FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleZoomToNaturalActionExecute));
	

	ToolkitCommands->MapAction(
		Commands.CheckeredBackground,
		FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleCheckeredBackgroundActionExecute, TextureEditorBackground_Checkered),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FTextureEditorToolkit::HandleCheckeredBackgroundActionIsChecked, TextureEditorBackground_Checkered));

	ToolkitCommands->MapAction(
		Commands.CheckeredBackgroundFill,
		FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleCheckeredBackgroundActionExecute, TextureEditorBackground_CheckeredFill),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FTextureEditorToolkit::HandleCheckeredBackgroundActionIsChecked, TextureEditorBackground_CheckeredFill));

	ToolkitCommands->MapAction(
		Commands.SolidBackground,
		FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleCheckeredBackgroundActionExecute, TextureEditorBackground_SolidColor),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FTextureEditorToolkit::HandleCheckeredBackgroundActionIsChecked, TextureEditorBackground_SolidColor));

	ToolkitCommands->MapAction(
		Commands.TextureBorder,
		FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleTextureBorderActionExecute),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FTextureEditorToolkit::HandleTextureBorderActionIsChecked));

	ToolkitCommands->MapAction(
		Commands.DefaultSampling,
		FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleSamplingActionExecute, TextureEditorSampling_Default),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FTextureEditorToolkit::HandleSamplingActionIsChecked, TextureEditorSampling_Default));

	ToolkitCommands->MapAction(
		Commands.PointSampling,
		FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleSamplingActionExecute, TextureEditorSampling_Point),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FTextureEditorToolkit::HandleSamplingActionIsChecked, TextureEditorSampling_Point));

	ToolkitCommands->MapAction(
		Commands.CompressNow,
		FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleCompressNowActionExecute),
		FCanExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleCompressNowActionCanExecute));

	ToolkitCommands->MapAction(
		Commands.Reimport,
		FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleReimportActionExecute),
		FCanExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleReimportActionCanExecute));

	ToolkitCommands->MapAction(
		Commands.Settings,
		FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleSettingsActionExecute));
}


TSharedRef<SWidget> FTextureEditorToolkit::BuildTexturePropertiesWidget( )
{
	FDetailsViewArgs Args;
	Args.bHideSelectionTip = true;

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TexturePropertiesWidget = PropertyModule.CreateDetailView(Args);
	TexturePropertiesWidget->SetObject(Texture);
	
	// todo, maybe :
	// check cvar "r.VT.MenuRestricted"
	// if set, hide the "VirtualTextureStreaming" property
	// TexturePropertiesWidget->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateSP(this, &FTextureEditorToolkit::ShowTextureProperty));

	return TexturePropertiesWidget.ToSharedRef();
}

// Manages the slate widgets for a single entry in the list of platforms and their formats.
class SPlatformTextureFormatRow : public SMultiColumnTableRow<TSharedRef<FTextureEditorToolkit::FPlatformTextureFormats>>
{
public:
	SLATE_BEGIN_ARGS(SPlatformTextureFormatRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, const TSharedPtr<FTextureEditorToolkit::FPlatformTextureFormats>& InNode)
	{
		Node = InNode;

		FSuperRowType::FArguments StyleArguments = FSuperRowType::FArguments()
		.Padding(1.0f)
		.Style(&FAppStyle().GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow"));

		SMultiColumnTableRow::Construct(StyleArguments, InOwnerTable);
	}

	FText GenerateMipSizeToolTip() const
	{
		TStringBuilder<64> MipSizes;
		int32 Count = 0;
		for (uint64 MipSize : Node->MipSizesInBytes)
		{
			if (Count)
			{
				MipSizes << TEXT('\n');
			}
			Count++;

			if (Node->NumMips_Tail && Count == Node->MipSizesInBytes.Num())
			{
				MipSizes << Count << TEXTVIEW("..") << Count + Node->NumMips_Tail << TEXTVIEW(" - ");
			}
			else
			{
				MipSizes << Count << TEXTVIEW(" - ");
			}

			MipSizes << FText::AsNumber(MipSize).ToString();

			if (Count <= Node->LODBias)
			{
				MipSizes << TEXTVIEW(" X");
			}
			else if (Count <= Node->LODBias + Node->NumMips_Optional)
			{
				MipSizes << TEXTVIEW(" O");
			}
			else if (Count <= Node->LODBias + Node->NumMips_Optional + Node->NumMips_Streaming)
			{
				MipSizes << TEXTVIEW(" S");
			}
			else
			{
				MipSizes << TEXTVIEW(" I");
			}
		}

		return FText::AsCultureInvariant(MipSizes.ToString());
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		if (InColumnName == PlatformsColumnId_Platform)
		{
			return SNew(STextBlock).Text(FText::FromName(Node->PlatformName)).Margin(FMargin(2, 0, 2, 0));
		}
		else if (InColumnName == PlatformsColumnId_TextureFormat)
		{
			return SNew(STextBlock).Text(FText::FromName(Node->TextureFormatName)).Margin(FMargin(2, 0, 2, 0));
		}
		else if (InColumnName == PlatformsColumnId_LODBias)
		{
			if (Node->bIsVirtual)
			{
				return SNew(STextBlock);
			}
			return SNew(STextBlock).Text(FText::AsNumber(Node->LODBias)).Margin(FMargin(2, 0, 2, 0)).Justification(ETextJustify::Right);
		}
		else if (InColumnName == PlatformsColumnId_NumMips)
		{
			if (Node->bIsVirtual)
			{
				return SNew(STextBlock);
			}
			TStringBuilder<32> Mips;
			Mips.Appendf(TEXT("%d / %d / %d / %d"), Node->NumMips_TotalAfterLOD, Node->NumMips_Optional, Node->NumMips_Streaming, Node->NumMips_Tail);
			return SNew(STextBlock)
				.Text(FText::AsCultureInvariant(Mips.ToString()))
				.ToolTipText(NSLOCTEXT("TextureEditor", "MipInfoTT", "Total After LODBias / Optional / Streaming / PackedTail"))
				.Margin(FMargin(2, 0, 2, 0));
		}
		else if (InColumnName == PlatformsColumnId_Dims)
		{
			if (Node->bIsVirtual)
			{
				return SNew(STextBlock).Text(NSLOCTEXT("TextureEditor", "CantBecauseVT", "N/A - VirtualTexture"));
			}
			TStringBuilder<32> Dims;
			Dims << Node->Width << TEXT('x') << Node->Height;
			return SNew(STextBlock).Text(FText::AsCultureInvariant(Dims.ToString())).Margin(FMargin(2, 0, 2, 0));
		}
		else if (InColumnName == PlatformsColumnId_SizeOptional)
		{
			if (Node->bIsVirtual)
			{
				return SNew(STextBlock);
			}
			return SNew(STextBlock)
				.Text(FText::AsNumber(Node->MipChainSizeInBytes_Optional))
				.ToolTipText(GenerateMipSizeToolTip())
				.Margin(FMargin(2, 0, 2, 0))
				.Justification(ETextJustify::Right);
		}
		else if (InColumnName == PlatformsColumnId_SizeInline)
		{
			if (Node->bIsVirtual)
			{
				return SNew(STextBlock);
			}
			return SNew(STextBlock)
				.Text(FText::AsNumber(Node->MipChainSizeInBytes_Inline))
				.ToolTipText(GenerateMipSizeToolTip())
				.Margin(FMargin(2, 0, 2, 0))
				.Justification(ETextJustify::Right);
		}
		else if (InColumnName == PlatformsColumnId_SizeStreaming)
		{
			if (Node->bIsVirtual)
			{
				return SNew(STextBlock);
			}
			return SNew(STextBlock)
				.Text(FText::AsNumber(Node->MipChainSizeInBytes_Streaming))
				.ToolTipText(GenerateMipSizeToolTip())
				.Margin(FMargin(2, 0, 2, 0))
				.Justification(ETextJustify::Right);
		}
		else
		{
			// Auto* formats will automatically use an opaque pixel format if not needed. We might or might not know
			// at this point which will get chosen - and it might not matter.
			if (Node->PixelFormat_AlphaTrue == Node->PixelFormat_AlphaFalse)
			{
				return SNew(STextBlock).Text(FText::AsCultureInvariant(GetPixelFormatString(Node->PixelFormat_AlphaFalse))).Margin(FMargin(2, 0, 2, 0));
			}

			if (Node->bKnowAlphaTransparency)
			{
				EPixelFormat PF = Node->PixelFormat_AlphaFalse;
				if (Node->bHasTransparentAlpha)
				{
					PF = Node->PixelFormat_AlphaTrue;
				}
				return SNew(STextBlock).Text(FText::AsCultureInvariant(GetPixelFormatString(PF))).Margin(FMargin(2, 0, 2, 0));
			}

			// If we don't know, show both. They can resolve this by clicking "Detect" in the "Details" page.
			TStringBuilder<128> Str;
			Str << GetPixelFormatString(Node->PixelFormat_AlphaFalse) << " or " << GetPixelFormatString(Node->PixelFormat_AlphaTrue);
			return SNew(STextBlock)
				.Text(FText::AsCultureInvariant(Str.ToString()))
				.ToolTipText(NSLOCTEXT("TextureEditor", "ResolveAlphaChannel", "We don't have alpha information in the source texture data - in the Details tab there is a \"Detect\" button in the quick info that can resolve this."))
				.Margin(FMargin(2, 0, 2, 0));
		}
	}

	TSharedPtr<FTextureEditorToolkit::FPlatformTextureFormats> Node;
};

TSharedRef<ITableRow> FTextureEditorToolkit::MakePlatformTextureFormatWidget(TSharedPtr<FPlatformTextureFormats> PlatformTextureFormat, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SPlatformTextureFormatRow, OwnerTable, PlatformTextureFormat);
}

void FTextureEditorToolkit::CreateInternalWidgets()
{
	//
	// Convert the packaging settings names into enums we can use.
	//
	UProjectPackagingSettings const* ProjectSettings = GetDefault<UProjectPackagingSettings>();
	
	PackagingSettingsNames.Add(MakeShared<FString>(TEXT("DebugDevelopment")));
	PackagingSettingsNames.Add(MakeShared<FString>(TEXT("TestShipping")));
	PackagingSettingsNames.Add(MakeShared<FString>(TEXT("Distribution")));
	
	// Default to Distribution
	TSharedPtr<FString> InitialPackagingSetting = PackagingSettingsNames[2];

	// Determine which oodle compressor they are using.
	const TCHAR* CompressorName = 0;
	{
		// Validity check the string by trying to convert to enum.
		const FString& LookupCompressor = ProjectSettings->PackageCompressionMethod;
		FOodleDataCompression::ECompressor PackageCompressor = FOodleDataCompression::ECompressor::Kraken;
		if (FOodleDataCompression::ECompressorFromString(LookupCompressor, PackageCompressor))
		{
			OodleCompressor = PackageCompressor;
		}
		else
		{
			UE_LOGF(LogTextureEditor, Warning, "Project packaging settings not using Oodle? Failed to recognize compression: %ls - using Kraken for estimation.", *LookupCompressor);
			OodleCompressor = FOodleDataCompression::ECompressor::Kraken;
		}

		FOodleDataCompression::ECompressorToString(OodleCompressor, &CompressorName);
	}

	OodleCompressionLevel = FOodleDataCompression::ECompressionLevel::Optimal3;
	const TCHAR* LevelName;
	{
		int8 PackageCompressionLevel_Distribution8 = 0;
		if (IntFitsIn<int8>(ProjectSettings->PackageCompressionLevel_Distribution))
		{
			PackageCompressionLevel_Distribution8 = static_cast<int8>(ProjectSettings->PackageCompressionLevel_Distribution);
		}
		else
		{
			UE_LOGF(LogTextureEditor, Warning, "PackageCompressionLevel_Distribution in Project Settings is an invalid value = %d, must be [-128, 127]", ProjectSettings->PackageCompressionLevel_Distribution);
		}
		FOodleDataCompression::ECompressionLevelFromValue(PackageCompressionLevel_Distribution8, OodleCompressionLevel);
		FOodleDataCompression::ECompressionLevelToString(OodleCompressionLevel, &LevelName);
	}

	// Grab the compression block size in the settings.
	{
		FString CompBlockSizeString;
		if (FParse::Value(*ProjectSettings->PackageAdditionalCompressionOptions, TEXT("-compressionblocksize="), CompBlockSizeString) &&
			FParse::Value(*ProjectSettings->PackageAdditionalCompressionOptions, TEXT("-compressionblocksize="), CompressionBlockSize))
		{
			if (CompBlockSizeString.EndsWith(TEXT("MB")))
			{
				CompressionBlockSize *= 1024 * 1024;
			}
			else if (CompBlockSizeString.EndsWith(TEXT("KB")))
			{
				CompressionBlockSize *= 1024;
			}
		}
		else
		{
			UE_LOGF(LogTextureEditor, Verbose, "No compression block size found in settings - using 64KB");
			CompressionBlockSize = 64*1024;
		}
	}

	TextureViewport = SNew(STextureEditorViewport, SharedThis(this));

	PlatformsTabContainer = SNew(SVerticalBox)
	+ SVerticalBox::Slot()
	.AutoHeight()
	.Padding(4.0f)
	[
		SAssignNew(PlatformTextureFormatsWidget, SListView<TSharedPtr<FPlatformTextureFormats>>)
		.ListItemsSource(&PlatformTextureFormatList)
		.SelectionMode(ESelectionMode::None)
		.HeaderRow(
			SNew(SHeaderRow)
			+SHeaderRow::Column(PlatformsColumnId_Platform)
			.DefaultLabel(LOCTEXT("TextureEditor_PlatformNameCol", "Platform"))
			.FillWidth(.2f)
			+SHeaderRow::Column(PlatformsColumnId_TextureFormat)
			.DefaultLabel(LOCTEXT("TextureEditor_TextureFormatCol", "Texture Format"))
			.FillWidth(.2f)
			+SHeaderRow::Column(PlatformsColumnId_PixelFormat)
			.DefaultLabel(LOCTEXT("TextureEditor_PixelFormatCol", "Pixel Format"))
			.DefaultTooltip(LOCTEXT("TextureEditor_PixelFormatColTT", "The pixel format that will be chosen for this platform. For texture formats that automatically select alpha or not (e.g. AutoDXT) this might not be known. Click \"Detect\" in the details panel to run the alpha scan and get the actual format"))
			.FillWidth(.15f)
			+SHeaderRow::Column(PlatformsColumnId_LODBias)
			.DefaultLabel(LOCTEXT("TextureEditor_LODBiasCol", "LODBias"))
			.DefaultTooltip(LOCTEXT("TextureEditor_LODBiasColTT", "The resolved LOD bias for this platform"))
			.FillWidth(.1f)
			+SHeaderRow::Column(PlatformsColumnId_Dims)
			.DefaultLabel(LOCTEXT("TextureEditor_DimsCol", "Size"))
			.DefaultTooltip(LOCTEXT("TextureEditor_DimsColTT", "The size of the top mip after LOD bias is applied."))
			.FillWidth(.1f)
			+SHeaderRow::Column(PlatformsColumnId_NumMips)
			.DefaultLabel(LOCTEXT("TextureEditor_NumMipsCol", "Mips"))
			.DefaultTooltip(LOCTEXT("TextureEditor_NumMipsColTT", "The number of mips after LOD bias, with the associated number of mips assigned optional, streaming, and inline. Additionally, if the platform has packed mip tails the number of mips in that mip tail is shown."))
			.FillWidth(.1f)
			+SHeaderRow::Column(PlatformsColumnId_SizeOptional)
			.DefaultLabel(LOCTEXT("TextureEditor_MipBytesSizeCol_Optional", "Optional"))
			.DefaultTooltip(LOCTEXT("TextureEditor_MipBytesSizeCol_OptionalTT", "The number of bytes of optional mips"))
			.FillWidth(.15f)
			+SHeaderRow::Column(PlatformsColumnId_SizeStreaming)
			.DefaultLabel(LOCTEXT("TextureEditor_MipBytesSizeCol_Streaming", "Streaming"))
			.DefaultTooltip(LOCTEXT("TextureEditor_MipBytesSizeCol_StreamingTT", "The number of bytes of streaming mips"))
			.FillWidth(.15f)
			+SHeaderRow::Column(PlatformsColumnId_SizeInline)
			.DefaultLabel(LOCTEXT("TextureEditor_MipBytesSizeCol_Inline", "Inline"))
			.DefaultTooltip(LOCTEXT("TextureEditor_MipBytesSizeCol_InlineTT", "The number of bytes of inline mips. If this is larger than you expect it's likely due to either platform alignment requirements or packed mip tails."))
			.FillWidth(.15f)
	)
		.OnGenerateRow(this, &FTextureEditorToolkit::MakePlatformTextureFormatWidget)
	]; // end format details.

	OodleTabContainer = SNew(SVerticalBox)

	//
	// Oodle relevant details container
	//
	+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f)
		[
			SNew(SHorizontalBox)

			//
			// Details label container
			//
			+ SHorizontalBox::Slot()
				.FillWidth(0.5f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(4.0f)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("OodleTab_Label_Encoder", "Encoder:"))
								.ToolTipText(LOCTEXT("OodleTab_Tooltip_Encoder", "Which texture encoder was used to encode the texture."))
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(4.0f)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("OodleTab_Label_EncodeSpeed", "Encode Speed:"))
								.ToolTipText(LOCTEXT("OodleTab_Tooltip_EncodeSpeed", "Which of the encode speeds was used for this texture encode, if the encoder supports encode speed."))
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(4.0f)
						[
							SAssignNew(OodleRDOEnabledLabel, STextBlock)
								.Text(LOCTEXT("OodleTab_Label_RDOEnabled", "RDO Lambda:"))
								.ToolTipText(LOCTEXT("OodleTab_Tooltip_RDOEnabled", "Whether or not the texture was encoded with RDO enabled. If enabled, shows the lambda used to encode. Excludes any global ini specific adjustments (e.g. GlobalLambdaMultiplier)"))
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(4.0f)
						[
							SAssignNew(OodleRDOSourceLabel, STextBlock)
								.Text(LOCTEXT("OodleTab_Label_RDOSource", "RDO Lambda Source:"))
								.ToolTipText(LOCTEXT("OodleTab_Tooltip_RDOSource", "This is where the build system found the lambda to use, due to defaults and fallbacks. (Lambda) means a direct lambda value (Lossy Compression Amount) means it was converted from that property."))
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(4.0f)
						[
							SAssignNew(OodleEffortLabel, STextBlock)
								.Text(LOCTEXT("OodleTab_Label_Effort", "Effort:"))
								.ToolTipText(LOCTEXT("OodleTab_ToolTip_Effort", "Which effort value was used when encoding this texture. Pulled from the encode speed options. Effort represents how much CPU time was spent finding better results."))
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(4.0f)
						[
							SAssignNew(OodleTilingLabel, STextBlock)
								.Text(LOCTEXT("OodleTab_Label_UniversalTiling", "Universal Tiling:"))
								.ToolTipText(LOCTEXT("OodleTab_ToolTip_UniversalTiling", "Which universal tiling setting was used when encoding this texture. Specified with encode speed. Universal Tiling is a technique to save on-disk space for platforms that expect tiled textures."))
						]
				]

			//
			// Details controls container
			//
			+ SHorizontalBox::Slot()
				.FillWidth(0.5f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(4.0f)
						[
							SAssignNew(OodleEncoderText, STextBlock)
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(4.0f)
						[
							SAssignNew(OodleEncodeSpeedText, STextBlock)
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(4.0f)
						[
							SAssignNew(OodleRDOText, STextBlock)
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(4.0f)
						[
							SAssignNew(OodleRDOSourceText, STextBlock)
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(4.0f)
						[
							SAssignNew(OodleEffortText, STextBlock)
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(4.0f)
						[
							SAssignNew(OodleTilingText, STextBlock)
						]
				]
		] // end oodle details.

	//
	// Header for oodle rdo experiments
	//
	+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		[
			SNew(SHeader)
				.HAlign(EHorizontalAlignment::HAlign_Fill)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("OodleTab_Label_TryHeader", "Try Encodings"))
				]
		]

	//
	// Container for oodle rdo experiments labels/controls
	//
	+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f)
		[
			//
			// Labels for oodle rdo experiments.
			//
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
				.FillWidth(0.5f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(6)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("OodleTab_Label_OverrideCompression", "Enabled:"))
							.ToolTipText(LOCTEXT("OodleTab_ToolTip_OverrideEncoding", "If checked, allows you to experiment with Oodle RDO encoder settings to visualize results."))
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(6)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("OodleTab_Label_OverrideRDO", "RDO Lambda:"))
							.ToolTipText(LOCTEXT("OodleTab_ToolTip_OverrideRDO", "The RDO lambda to encode with for experimentation. 0 disables RDO entirely. 1 is largest filesize, 100 is smallest."))
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(6)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("OodleTab_Label_OverrideEffort", "Effort:"))
							.ToolTipText(LOCTEXT("OodleTab_ToolTip_OverrideEffort", "The encoding effort to try. Effort controls how much CPU time spent on finding better results. See the Oodle Texture documentation for detailed information."))
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(6)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("OodleTab_Label_OverrideTiling", "Universal Tiling:"))
							.ToolTipText(LOCTEXT("OodleTab_ToolTip_OverrideTiling", "The universal tiling to try. See the Oodle Texture documentation for detailed information."))
						]
				]

			//
			// Controls for oodle rdo experiments
			//
			+ SHorizontalBox::Slot()
				.FillWidth(0.5f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(2)
						[
							SAssignNew(OodleOverrideCheck, SCheckBox)
							.OnCheckStateChanged(this, &FTextureEditorToolkit::OnUseEditorOodleSettingsChanged)
							.IsChecked(this, &FTextureEditorToolkit::UseEditorOodleSettingsChecked)
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(2)
						[
							SNew(SNumericEntryBox<int32>)
							.Value(this, &FTextureEditorToolkit::GetEditorOodleSettingsRDO)
							.OnValueCommitted(this, &FTextureEditorToolkit::EditorOodleSettingsRDOCommitted)
							.IsEnabled(this, &FTextureEditorToolkit::EditorOodleSettingsEnabled)
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(2)
						[
							SNew(SEnumComboBox, StaticEnum< ETextureEncodeEffort >())
							.CurrentValue(this, &FTextureEditorToolkit::GetEditorOodleSettingsEffort)
							.OnEnumSelectionChanged(this, &FTextureEditorToolkit::EditorOodleSettingsEffortChanged)
							.IsEnabled(this, &FTextureEditorToolkit::EditorOodleSettingsEnabled)
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(2)
						[
							SNew(SEnumComboBox, StaticEnum< ETextureUniversalTiling >())
							.CurrentValue(this, &FTextureEditorToolkit::GetEditorOodleSettingsTiling)
							.OnEnumSelectionChanged(this, &FTextureEditorToolkit::EditorOodleSettingsTilingChanged)
							.IsEnabled(this, &FTextureEditorToolkit::EditorOodleSettingsEnabled)
						]
				] // end controls
		] // end oodle rdo experiment labels/controls

	//
	// Header for the on disk estimates
	//
	+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		[
			SNew(SHeader)
			.HAlign(EHorizontalAlignment::HAlign_Fill)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("OodleTab_Label_EstimatesHeader", "On-disk Sizes"))
				.ToolTipText(LOCTEXT("OodleTab_ToolTip_EstimatesHeader", "RDO encoding only helps on-disk texture sizes when package compression is enabled. It does not affect runtime memory usage."))
			]
		]
	//
	// Container for the on disk estimate labels/controls.
	//
	+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f)
		[
			//
			// Labels for the on disk estimates
			//
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
				.FillWidth(0.5f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(6)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("OodleTab_Label_EstimatesEnabled", "Enabled:"))
							.ToolTipText(LOCTEXT("OodleTab_ToolTip_EstimatesEnabled", "If checked, texture data will be compressed in the same manner as project packaging in order to estimate the benefits of RDO encoding of the texture."))
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(6)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("OodleTab_Label_EncoderSettings", "Packaging Configuration:"))
							.ToolTipText(LOCTEXT("OodleTab_ToolTip_CompressorSettings", "Which packaging configuration to pull from for determining which Oodle compressor and compression level to use."))
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(6)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("OodleTab_Label_EstimateCompressor", "Oodle Compressor:"))
							.ToolTipText(LOCTEXT("OodleTab_ToolTip_EstimateCompressor", "The oodle compressor to use for estimating. Pulled from the packaging configuration specified above."))
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(6)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("OodleTab_Label_EstimateLevel", "Oodle Compression Level:"))
							.ToolTipText(LOCTEXT("OodleTab_ToolTip_EstimateLevel", "The compression level. Pulled from the packaging configuration specified above."))
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(6)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("OodleTab_Label_BlockSize", "Compression Block Size:"))
							.ToolTipText(LOCTEXT("OodleTab_ToolTip_BlockSize", "The size of chunks used when compressing. Pulled from the packaging configuration 'Package Compression Commandline Options'."))
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(6)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("OodleTab_Label_EstimateRaw", "Uncompressed size:"))
							.ToolTipText(LOCTEXT("OodleTab_ToolTip_EstimateRaw", "The size of the mip or virtual texture data for the texture."))
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(6)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("OodleTab_Label_EstimateCompressed", "Compressed size (estimate):"))
							.ToolTipText(LOCTEXT("OodleTab_ToolTip_EstimateCompressed", "The size of the compressed mip or virtual texture data for the texture."))
						]
				]

			//
			// Controls for the on disk estimates
			//
			+ SHorizontalBox::Slot()
				.FillWidth(0.5f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(2)
						[
							SAssignNew(OodleEstimateCheck, SCheckBox)
							.OnCheckStateChanged(this, &FTextureEditorToolkit::OnEstimateCompressionChanged)
							.IsChecked(this, &FTextureEditorToolkit::EstimateCompressionChecked)
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(2)
						[
							SNew(STextComboBox)
							.OptionsSource(&PackagingSettingsNames)
							.OnSelectionChanged(this, &FTextureEditorToolkit::PackagingSettingsChanged)
							.IsEnabled(this, &FTextureEditorToolkit::EstimateCompressionEnabled)
							.InitiallySelectedItem(InitialPackagingSetting)
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(8)
						[
							SAssignNew(OodleCompressorUsed, STextBlock)
							.Text(FText::AsCultureInvariant(CompressorName))
							.IsEnabled(this, &FTextureEditorToolkit::EstimateCompressionEnabled)
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(6)
						[
							SAssignNew(OodleLevelUsed, STextBlock)
							.Text(FText::FromString(FString::Printf(TEXT("%s (%d)"), LevelName, (int8)OodleCompressionLevel)))
							.IsEnabled(this, &FTextureEditorToolkit::EstimateCompressionEnabled)
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(6)
						[
							SAssignNew(OodleCompressionBlockUsed, STextBlock)
							.Text(FText::AsMemory(CompressionBlockSize))
							.IsEnabled(this, &FTextureEditorToolkit::EstimateCompressionEnabled)
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(6)
						[
							SAssignNew(OodleEstimateRaw, STextBlock)
							.IsEnabled(this, &FTextureEditorToolkit::EstimateCompressionEnabled)
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.Padding(6)
						[
							SAssignNew(OodleEstimateCompressed, STextBlock)
							.IsEnabled(this, &FTextureEditorToolkit::EstimateCompressionEnabled)
						]
				]
		]; // end on disk estimates controls


	TextureProperties = SNew(SVerticalBox)

	+ SVerticalBox::Slot()
	.AutoHeight()
	.Padding(2.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(0.5f)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(ImportedText, STextBlock)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(CurrentText, STextBlock)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(MaxInGameText, STextBlock)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(SizeText, STextBlock)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.FillWidth(1.f)
				[
					SAssignNew(SourceMipsAlphaDetectedText, STextBlock)
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.AutoWidth()
				[
					SAssignNew(DetectSourceAlphaButton, SButton)
					.Text(LOCTEXT("DetectSourceAlpha_Button", "Detect"))
					.OnClicked(this, &FTextureEditorToolkit::DetectSourceAlphaButton_Clicked)
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(SceneCaptureSizeText, STextBlock)
			]
		]

		+ SHorizontalBox::Slot()
		.FillWidth(0.5f)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(MethodText, STextBlock)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(FormatText, STextBlock)
			]


			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(HasAlphaChannelText, STextBlock)
			]


			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(LODBiasText, STextBlock)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(NumMipsText, STextBlock)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(EncodeSpeedText, STextBlock)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(SceneCaptureNameText, STextBlock)
			]
		]
	]

	+ SVerticalBox::Slot()
	.FillHeight(1.0f)
	.Padding(2.0f)
	[
		BuildTexturePropertiesWidget()
	];
}

void FTextureEditorToolkit::ExtendToolBar()
{
	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateSP(this, &FTextureEditorToolkit::FillToolbar)
	);

	AddToolbarExtender(ToolbarExtender);

	// Extensions are currently disabled in read-only mode, if they are desired to be added in the future we should move this code to the individual extensions
	if(GetOpenMethod() == EAssetOpenMethod::Edit)
	{
		ITextureEditorModule* TextureEditorModule = &FModuleManager::LoadModuleChecked<ITextureEditorModule>("TextureEditor");
		AddToolbarExtender(TextureEditorModule->GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
	}
}

void FTextureEditorToolkit::FillToolbar(FToolBarBuilder& ToolbarBuilder)
{
	TSharedRef<SWidget> ChannelControl = MakeChannelControlWidget();
	TSharedRef<SWidget> LODControl = MakeLODControlWidget();
	TSharedRef<SWidget> LayerControl = MakeLayerControlWidget();
	TSharedRef<SWidget> SliceControl = MakeSliceControlWidget();
	TSharedRef<SWidget> FaceControl = MakeFaceControlWidget();
	TSharedRef<SWidget> ExposureControl = MakeExposureContolWidget();
	TSharedPtr<SWidget> OptionalOpacityControl = IsVolumeTexture() ? TSharedPtr<SWidget>(MakeOpacityControlWidget()) : nullptr;
	TSharedRef<SWidget> ZoomControl = MakeZoomControlWidget();
	TSharedRef<SWidget> View3DControl = MakeView3DControlWidget();

	UCurveLinearColorAtlas* Atlas = Cast<UCurveLinearColorAtlas>(GetTexture());
	bool bCanRecompress = CanRecompressTexture(GetTexture());
	bool bIsAtlas = (Atlas != nullptr);
	
	if ( bCanRecompress && ! bIsAtlas )
	{
		// These actions don't make sense in read-only mode
		if(GetOpenMethod() == EAssetOpenMethod::Edit)
		{
			ToolbarBuilder.BeginSection("TextureMisc");
			{
				ToolbarBuilder.AddToolBarButton(FTextureEditorCommands::Get().CompressNow);
				ToolbarBuilder.AddToolBarButton(FTextureEditorCommands::Get().Reimport);
			}
			ToolbarBuilder.EndSection();
		}
	}

	if ( 1 )
	{
		ToolbarBuilder.BeginSection("Channels");
		{
			ToolbarBuilder.AddWidget(ChannelControl);
		}
		ToolbarBuilder.EndSection();

		ToolbarBuilder.BeginSection("TextureMipAndExposure");
		{
			ToolbarBuilder.AddWidget(LODControl);
			ToolbarBuilder.AddWidget(ExposureControl);
		}	
		ToolbarBuilder.EndSection();

		if (HasLayers())
		{
			ToolbarBuilder.BeginSection("Layers");
			{
				ToolbarBuilder.AddWidget(LayerControl);
			}
			ToolbarBuilder.EndSection();
		}

		if (HasSlices())
		{
			ToolbarBuilder.BeginSection("Slices");
			{
				ToolbarBuilder.AddWidget(SliceControl);
			}
			ToolbarBuilder.EndSection();
		}

		if (IsCubeTexture())
		{
			ToolbarBuilder.BeginSection("Faces");
			{
				ToolbarBuilder.AddWidget(FaceControl);
			}
			ToolbarBuilder.EndSection();
		}

		if (OptionalOpacityControl.IsValid())
		{
			ToolbarBuilder.BeginSection("Opacity");
			{
				ToolbarBuilder.AddWidget(OptionalOpacityControl.ToSharedRef());
			}
			ToolbarBuilder.EndSection();
		}

		if (IsCubeTexture() || IsVolumeTexture())
		{
			ToolbarBuilder.BeginSection("View3D");
			{
				ToolbarBuilder.AddWidget(View3DControl);
			}
			ToolbarBuilder.EndSection();
		}

		ToolbarBuilder.BeginSection("Zoom");
		{
			ToolbarBuilder.AddWidget(ZoomControl);
		}
		ToolbarBuilder.EndSection();

		{
			ToolbarBuilder.BeginSection("PlatformPreview");
			{
				ToolbarBuilder.AddWidget(MakePlatformSelectorWidget());
			}
			ToolbarBuilder.EndSection();
		}


		ToolbarBuilder.BeginSection("Settings");
		ToolbarBuilder.BeginStyleOverride("CalloutToolbar");
		{
			ToolbarBuilder.AddWidget(SNew(SSpacer), NAME_None, false, HAlign_Right);
			ToolbarBuilder.AddComboButton(
				FUIAction(),
				FOnGetContent::CreateSP(this, &FTextureEditorToolkit::OnGenerateSettingsMenu),
				LOCTEXT("SettingsMenu", "View Settings"),
				FText::GetEmpty(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Settings")
			);
		}
		ToolbarBuilder.EndStyleOverride();
		ToolbarBuilder.EndSection();
	}
}

TOptional<int32> FTextureEditorToolkit::GetMaxMipLevel() const
{
	TOptional<int32> MaxMipLevel;
	int32 NumMips = GetNumMips();
	if (NumMips > 0)
	{
		MaxMipLevel = NumMips - 1;
	}
	return MaxMipLevel;
}

int32 FTextureEditorToolkit::GetNumMips( ) const
{
	if (const UTexture2D* Texture2D = Cast<UTexture2D>(Texture))
	{
		return Texture2D->GetNumMips();
	}
	else if (const UTextureCube* TextureCube = Cast<UTextureCube>(Texture))
	{
		return TextureCube->GetNumMips();
	}
	else if (const UTextureCubeArray* TextureCubeArray = Cast<UTextureCubeArray>(Texture))
	{
		return TextureCubeArray->GetNumMips();
	}
	else if (const UTexture2DArray* Texture2DArray = Cast<UTexture2DArray>(Texture))
	{
		return Texture2DArray->GetNumMips();
	}
	else if (const UVolumeTexture* VolumeTexture = Cast<UVolumeTexture>(Texture))
	{
		return VolumeTexture->GetNumMips();
	}
	else if (const UTextureRenderTarget2D* Texture2DRT = Cast<UTextureRenderTarget2D>(Texture))
	{
		return Texture2DRT->GetNumMips();
	}
	else if (const UTextureRenderTargetCube* TextureCubeRT = Cast<UTextureRenderTargetCube>(Texture))
	{
		return TextureCubeRT->GetNumMips();
	}
	else if (const UTextureRenderTarget2DArray* Texture2DArrayRT = Cast<UTextureRenderTarget2DArray>(Texture))
	{
		return Texture2DArrayRT->GetNumMips();
	}
	else if (const UTextureRenderTargetVolume* VolumeTextureRT = Cast<UTextureRenderTargetVolume>(Texture))
	{
		return VolumeTextureRT->GetNumMips();
	}
	else if (const UTexture2DDynamic* Texture2DDynamic = Cast<UTexture2DDynamic>(Texture))
	{
		return Texture2DDynamic->NumMips;
	}
	else if (const UMediaTexture* MediaTexture = Cast<UMediaTexture>(Texture))
	{
		return MediaTexture->GetTextureNumMips();
	}

	return MIPLEVEL_MAX;
}

EPixelFormat FTextureEditorToolkit::GetPixelFormat() const
{
	if (const UTexture2D* Texture2D = Cast<UTexture2D>(Texture))
	{
		return Texture2D->GetPixelFormat(SpecifiedLayer);
	}
	else if (const UTextureCube* TextureCube = Cast<UTextureCube>(Texture))
	{
		return TextureCube->GetPixelFormat();
	}
	else if (const UTexture2DArray* Texture2DArray = Cast<UTexture2DArray>(Texture))
	{
		return Texture2DArray->GetPixelFormat();
	}
	else if (const UTextureCubeArray* TextureCubeArray = Cast<UTextureCubeArray>(Texture))
	{
		return TextureCubeArray->GetPixelFormat();
	}
	else if (const UVolumeTexture* VolumeTexture = Cast<UVolumeTexture>(Texture))
	{
		return VolumeTexture->GetPixelFormat();
	}
	else if (const UTextureRenderTarget2D* Texture2DRT = Cast<UTextureRenderTarget2D>(Texture))
	{
		return Texture2DRT->GetFormat();
	}
	else if (const UTextureRenderTargetCube* TextureCubeRT = Cast<UTextureRenderTargetCube>(Texture))
	{
		return TextureCubeRT->GetFormat();
	}
	else if (const UTextureRenderTarget2DArray* Texture2DArrayRT = Cast<UTextureRenderTarget2DArray>(Texture))
	{
		return Texture2DArrayRT->GetFormat();
	}
	else if (const UTextureRenderTargetVolume* VolumeTextureRT = Cast<UTextureRenderTargetVolume>(Texture))
	{
		return VolumeTextureRT->GetFormat();
	}
	else if (const UTexture2DDynamic* Texture2DDynamic = Cast<UTexture2DDynamic>(Texture))
	{
		return Texture2DDynamic->Format;
	}
	//else if (const UMediaTexture* MediaTexture = Cast<UMediaTexture>(Texture))
	//{
	//	UMediaTexture::GetDesc() suggests PF_B8G8R8A8, maybe?
	//}

	return PF_MAX;
}

TOptional<int32> FTextureEditorToolkit::GetMaxLayer() const
{
	return FMath::Max(Texture->Source.GetNumLayers() - 1, 1);
}

TOptional<int32> FTextureEditorToolkit::GetMaxSlice() const
{
	return FMath::Max(0, GetNumSlices() - 1);
}

bool FTextureEditorToolkit::IsCubeTexture( ) const
{
	return (Texture->IsA(UTextureCube::StaticClass()) || Texture->IsA(UTextureCubeArray::StaticClass()) || Texture->IsA(UTextureRenderTargetCube::StaticClass()));
}

bool FTextureEditorToolkit::IsVolumeTexture() const
{
	return (Texture->IsA(UVolumeTexture::StaticClass()) || Texture->IsA(UTextureRenderTargetVolume::StaticClass()));
}

bool FTextureEditorToolkit::Is2DArrayTexture() const
{
	return (Texture->IsA(UTexture2DArray::StaticClass()) || Texture->IsA(UTextureRenderTarget2DArray::StaticClass()));
}

bool FTextureEditorToolkit::IsArrayTexture() const
{
	return Is2DArrayTexture() || Texture->IsA(UTextureCubeArray::StaticClass());
}

TSharedRef<SWidget> FTextureEditorToolkit::OnGenerateMipMapLevelMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	for (int32 MipLevel = MIPLEVEL_MIN; MipLevel <= GetMaxMipLevel().Get(MIPLEVEL_MAX); ++MipLevel)
	{
		FText MipNumberText = FText::AsNumber(MipLevel);

		MenuBuilder.AddMenuEntry(
			FText::Format(LOCTEXT("MipLevel", "Mip Level {0}"), MipNumberText),
			FText::Format(LOCTEXT("MipLevel_Tooltip", "Display Mip Level {0}"), MipNumberText),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleMipLevelChanged, MipLevel),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, MipLevel]() {return SpecifiedMipLevel == MipLevel; })
			)
		);
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FTextureEditorToolkit::OnGenerateSettingsMenu()
{
	FMenuBuilder MenuBuilder(true, ToolkitCommands);
	FTextureEditorViewOptionsMenu::MakeMenu(MenuBuilder);

	return MenuBuilder.MakeWidget();
}

/* FTextureEditorToolkit callbacks
 *****************************************************************************/
bool FTextureEditorToolkit::IsChannelButtonEnabled(ETextureChannelButton Button) const
{
	EPixelFormatChannelFlags ValidTextureChannels = GetPixelFormatValidChannels(GetPixelFormat());
	return EnumHasAnyFlags(ValidTextureChannels, GetPixelFormatChannelFlagForButton(Button));
}

FSlateColor FTextureEditorToolkit::GetChannelButtonBackgroundColor(ETextureChannelButton Button) const
{
	FSlateColor Dropdown = FAppStyle::Get().GetSlateColor("Colors.Dropdown");

	switch (Button)
	{
	case ETextureChannelButton::Red:
		return bIsRedChannel ? FLinearColor::Red : FLinearColor::White;
	case ETextureChannelButton::Green:
		return bIsGreenChannel ? FLinearColor::Green : FLinearColor::White;
	case ETextureChannelButton::Blue:
		return bIsBlueChannel ? FLinearColor::Blue : FLinearColor::White;
	case ETextureChannelButton::Alpha:
		return FLinearColor::White;
	default:
		check(false);
		return FSlateColor();
	}
}

FSlateColor FTextureEditorToolkit::GetChannelButtonForegroundColor(ETextureChannelButton Button) const
{
	FSlateColor DefaultForeground = FAppStyle::Get().GetSlateColor("Colors.Foreground");

	switch (Button)
	{
	case ETextureChannelButton::Red:
		return bIsRedChannel ? FLinearColor::Black : DefaultForeground;
	case ETextureChannelButton::Green:
		return bIsGreenChannel ? FLinearColor::Black : DefaultForeground;
	case ETextureChannelButton::Blue:
		return bIsBlueChannel ? FLinearColor::Black : DefaultForeground;
	case ETextureChannelButton::Alpha:
		return bIsAlphaChannel ? FLinearColor::Black : DefaultForeground;
	default:
		check(false);
		return FSlateColor::UseForeground();
	}
}

void FTextureEditorToolkit::OnChannelButtonCheckStateChanged(ETextureChannelButton Button)
{
	switch (Button)
	{
	case ETextureChannelButton::Red:
		bIsRedChannel = !bIsRedChannel;
		break;
	case ETextureChannelButton::Green:
		bIsGreenChannel = !bIsGreenChannel;
		break;
	case ETextureChannelButton::Blue:
		bIsBlueChannel = !bIsBlueChannel;
		break;
	case ETextureChannelButton::Alpha:
		bIsAlphaChannel = !bIsAlphaChannel;
		break;
	default:
		check(false);
		break;
	}
}

ECheckBoxState FTextureEditorToolkit::OnGetChannelButtonCheckState(ETextureChannelButton Button) const
{
	switch (Button)
	{
	case ETextureChannelButton::Red:
		return bIsRedChannel ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	case ETextureChannelButton::Green:
		return bIsGreenChannel ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	case ETextureChannelButton::Blue:
		return bIsBlueChannel ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	case ETextureChannelButton::Alpha:
		return bIsAlphaChannel ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	default:
		check(false);
		return ECheckBoxState::Unchecked;
	}
}


void FTextureEditorToolkit::HandleCheckeredBackgroundActionExecute( ETextureEditorBackgrounds Background )
{
	UTextureEditorSettings& Settings = *GetMutableDefault<UTextureEditorSettings>();
	Settings.Background = Background;
	Settings.PostEditChange();
}


bool FTextureEditorToolkit::HandleCheckeredBackgroundActionIsChecked( ETextureEditorBackgrounds Background )
{
	const UTextureEditorSettings& Settings = *GetDefault<UTextureEditorSettings>();

	return (Background == Settings.Background);
}

void FTextureEditorToolkit::HandleSamplingActionExecute(ETextureEditorSampling InSampling)
{
	// Update our own sampling
	Sampling = InSampling;

	UTextureEditorSettings& Settings = *GetMutableDefault<UTextureEditorSettings>();
	Settings.Sampling = InSampling;
	Settings.PostEditChange();
}


bool FTextureEditorToolkit::HandleSamplingActionIsChecked(ETextureEditorSampling InSampling)
{
	return InSampling == Sampling;
}

// Callback for toggling the volume view action.
void FTextureEditorToolkit::HandleVolumeViewModeActionExecute(ETextureEditorVolumeViewMode InViewMode)
{
	UTextureEditorSettings& Settings = *GetMutableDefault<UTextureEditorSettings>();
	Settings.VolumeViewMode = InViewMode;
	Settings.PostEditChange();
}

// Callback for getting the checked state of the volume view action.
bool FTextureEditorToolkit::HandleVolumeViewModeActionIsChecked(ETextureEditorVolumeViewMode InViewMode)
{
	const UTextureEditorSettings& Settings = *GetDefault<UTextureEditorSettings>();

	return (InViewMode == Settings.VolumeViewMode);
}

// Callback for toggling the cubemap view action.
void FTextureEditorToolkit::HandleCubemapViewModeActionExecute(ETextureEditorCubemapViewMode InViewMode)
{
	UTextureEditorSettings& Settings = *GetMutableDefault<UTextureEditorSettings>();
	Settings.CubemapViewMode = InViewMode;
	Settings.PostEditChange();
}

// Callback for getting the checked state of the cubemap view action.
bool FTextureEditorToolkit::HandleCubemapViewModeActionIsChecked(ETextureEditorCubemapViewMode InViewMode)
{
	UTextureEditorSettings& Settings = *GetMutableDefault<UTextureEditorSettings>();
	return Settings.CubemapViewMode == InViewMode;
}

void FTextureEditorToolkit::HandleCompressNowActionExecute( )
{
	if ( ! CanRecompressTexture(Texture) )
	{
		return;
	}

	GWarn->BeginSlowTask(NSLOCTEXT("TextureEditor", "CompressNow", "Compressing 1 Textures that have Defer Compression set"), true);

	// turn off deferred compression and compress the texture

	if (Texture->DeferCompression)
	{
		Texture->BlockOnAnyAsyncBuild(); // PreEditChange , but don't mark as modified ; same as Modify(false)
		Texture->DeferCompression = false;
		//Texture->Source.Compress(); // <- done in UTexture::PreSave
		Texture->PostEditChange();

		PopulateQuickInfo();
	}

	GWarn->EndSlowTask();
}


bool FTextureEditorToolkit::HandleCompressNowActionCanExecute( ) const
{
	return (Texture->DeferCompression != 0);
}


void FTextureEditorToolkit::HandleFitToViewportActionExecute( )
{
	SetZoomMode(ETextureEditorZoomMode::Fit);
}


void FTextureEditorToolkit::HandleFillToViewportActionExecute()
{
	SetZoomMode(ETextureEditorZoomMode::Fill);
}

void FTextureEditorToolkit::HandleZoomToNaturalActionExecute()
{
	SetCustomZoomLevel(1);
}

void FTextureEditorToolkit::HandleMipLevelCheckBoxCheckedStateChanged( ECheckBoxState InNewState )
{
	bUseSpecifiedMipLevel = InNewState == ECheckBoxState::Checked;
}


ECheckBoxState FTextureEditorToolkit::HandleMipLevelCheckBoxIsChecked( ) const
{
	return GetUseSpecifiedMip() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}


bool FTextureEditorToolkit::HandleMipLevelCheckBoxIsEnabled( ) const
{
	return GetMaxMipLevel().Get(MIPLEVEL_MAX) > 0;
}

void FTextureEditorToolkit::HandleMipLevelChanged(int32 NewMipLevel)
{
	SpecifiedMipLevel = FMath::Clamp<int32>(NewMipLevel, MIPLEVEL_MIN, GetMaxMipLevel().Get(MIPLEVEL_MAX));
	PopulateQuickInfo(); // so PreviewEffectiveTexture{Width,Height} get updated immediately

	MipLevelTextBlock->SetText(FText::Format(LOCTEXT("MipLevel", "Mip Level {0}"), SpecifiedMipLevel));
}

TOptional<int32> FTextureEditorToolkit::HandleMipLevelEntryBoxValue( ) const
{
	return SpecifiedMipLevel;
}

FReply FTextureEditorToolkit::HandleMipMapMinusButtonClicked()
{
	HandleMipLevelChanged(--SpecifiedMipLevel);

	return FReply::Handled();
}

FReply FTextureEditorToolkit::HandleMipMapPlusButtonClicked()
{
	HandleMipLevelChanged(++SpecifiedMipLevel);

	return FReply::Handled();
}

void FTextureEditorToolkit::HandleLayerEntryBoxChanged(int32 NewLayer)
{
	SpecifiedLayer = FMath::Clamp<int32>(NewLayer, 0, Texture->Source.GetNumLayers() - 1);
	PopulateQuickInfo();
}


TOptional<int32> FTextureEditorToolkit::HandleLayerEntryBoxValue() const
{
	return SpecifiedLayer;
}

bool FTextureEditorToolkit::HasLayers() const
{
	return Texture->Source.GetNumLayers() > 1;
}

bool FTextureEditorToolkit::HasSlices() const
{
	// slice selection should be supported even for a texture array with less than two elements, because array elements can be added dynamically
	return IsArrayTexture();
}

int32 FTextureEditorToolkit::GetNumSlices() const
{
	if (Texture->IsA(UTexture2DArray::StaticClass()))
	{
		return Cast<UTexture2DArray>(Texture)->GetArraySize();
	}
	else if (Texture->IsA(UTextureRenderTarget2DArray::StaticClass()))
	{
		return Cast<UTextureRenderTarget2DArray>(Texture)->GetSurfaceArraySize();
	}
	else if (Texture->IsA(UTextureCubeArray::StaticClass()))
	{
		// for a TextureCube array SliceIndex represents an index of a cubemap in the array
		return Cast<UTextureCubeArray>(Texture)->GetSurfaceArraySize() / 6;
	}
	return 1;
}

bool FTextureEditorToolkit::HandleSliceCheckBoxIsEnabled() const
{
	return GetNumSlices() > 1;
}

ECheckBoxState FTextureEditorToolkit::HandleSliceCheckBoxIsChecked() const
{
	return GetUseSpecifiedSlice() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FTextureEditorToolkit::HandleSliceCheckBoxCheckedStateChanged(ECheckBoxState InNewState)
{
	bUseSpecifiedSlice = InNewState == ECheckBoxState::Checked;
}

TOptional<int32> FTextureEditorToolkit::HandleSliceEntryBoxValue() const
{
	return FMath::Clamp<int32>(SpecifiedSlice, 0, GetMaxSlice().Get(0));
}

void FTextureEditorToolkit::HandleSliceEntryBoxChanged(int32 NewSlice)
{
	SpecifiedSlice = FMath::Clamp<int32>(NewSlice, 0, GetMaxSlice().Get(0));
	PopulateQuickInfo();
}

ECheckBoxState FTextureEditorToolkit::HandleFaceCheckBoxIsChecked() const
{
	return GetUseSpecifiedFace() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FTextureEditorToolkit::HandleFaceCheckBoxCheckedStateChanged(ECheckBoxState InNewState)
{
	bUseSpecifiedFace = InNewState == ECheckBoxState::Checked;
}

TOptional<int32> FTextureEditorToolkit::HandleFaceEntryBoxValue() const
{
	return FMath::Clamp<int32>(SpecifiedFace, 0, 5);
}

void FTextureEditorToolkit::HandleFaceEntryBoxChanged(int32 NewFace)
{
	SpecifiedFace = FMath::Clamp<int32>(NewFace, 0, 5);
	PopulateQuickInfo();
}

ECheckBoxState FTextureEditorToolkit::HandleView3DCheckBoxIsChecked() const
{
	if (IsVolumeTexture())
	{
		return GetVolumeViewMode() == TextureEditorVolumeViewMode_VolumeTrace ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	else
	{
		check(IsCubeTexture());
		return GetCubemapViewMode() == TextureEditorCubemapViewMode_3DView ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
}

void FTextureEditorToolkit::HandleView3DCheckBoxCheckedStateChanged(ECheckBoxState InNewState)
{
	if (IsVolumeTexture())
	{
		SetVolumeViewMode(InNewState == ECheckBoxState::Checked ? TextureEditorVolumeViewMode_VolumeTrace : TextureEditorVolumeViewMode_DepthSlices);
	}
	else
	{
		check(IsCubeTexture());
		SetCubemapViewMode(InNewState == ECheckBoxState::Checked ? TextureEditorCubemapViewMode_3DView : TextureEditorCubemapViewMode_2DView);
	}	
}

bool FTextureEditorToolkit::HandleReimportActionCanExecute( ) const
{
	if (Texture->IsA<ULightMapTexture2D>() || Texture->IsA<UShadowMapTexture2D>() || Texture->IsA<UTexture2DDynamic>() || Texture->IsA<UTextureRenderTarget>() || Texture->IsA<UCurveLinearColorAtlas>())
	{
		return false;
	}

	return true;
}


void FTextureEditorToolkit::HandleReimportActionExecute( )
{
	FReimportManager::Instance()->ReimportAsync(Texture, /*bAskForNewFileIfMissing=*/true);
}


void FTextureEditorToolkit::HandleReimportManagerPostReimport( UObject* InObject, bool bSuccess )
{
	// Ignore if this is regarding a different object
	if (InObject != Texture)
	{
		return;
	}

	// Re-enable viewport rendering now that the texture should be in a known state again
	TextureViewport->EnableRendering();
}


void FTextureEditorToolkit::HandleReimportManagerPreReimport( UObject* InObject )
{
	// Ignore if this is regarding a different object
	if (InObject != Texture)
	{
		return;
	}

	// Disable viewport rendering until the texture has finished re-importing
	TextureViewport->DisableRendering();
}

void FTextureEditorToolkit::HandleAssetPostImport(UFactory* InFactory, UObject* InObject)
{
	if (Cast<UTexture>(InObject) != nullptr && InObject == Texture && TexturePropertiesWidget != nullptr )
	{
		// Refresh this object within the details panel
		TexturePropertiesWidget->SetObject(InObject);
	}
}

void FTextureEditorToolkit::HandleDesaturationChannelActionExecute( )
{
	bIsDesaturation = !bIsDesaturation;
}


bool FTextureEditorToolkit::HandleDesaturationChannelActionIsChecked( ) const
{
	return bIsDesaturation;
}


void FTextureEditorToolkit::HandleSettingsActionExecute( )
{
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Editor", "ContentEditors", "TextureEditor");
}


TSharedRef<SDockTab> FTextureEditorToolkit::HandleTabSpawnerSpawnPlatforms(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == PlatformsTabId);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
	.Label(LOCTEXT("TexturePlatformsTitle", "Platforms"))
	[
		PlatformsTabContainer.ToSharedRef()
	];

	return SpawnedTab;
}

TSharedRef<SDockTab> FTextureEditorToolkit::HandleTabSpawnerSpawnOodle(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == OodleTabId);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("TextureOodleTitle", "Oodle"))
		[
			OodleTabContainer.ToSharedRef()
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FTextureEditorToolkit::HandleTabSpawnerSpawnProperties( const FSpawnTabArgs& Args )
{
	check(Args.GetTabId() == PropertiesTabId);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("TexturePropertiesTitle", "Details"))
		[
			TextureProperties.ToSharedRef()
		];

	PopulateQuickInfo();

	return SpawnedTab;
}


TSharedRef<SDockTab> FTextureEditorToolkit::HandleTabSpawnerSpawnViewport( const FSpawnTabArgs& Args )
{
	check(Args.GetTabId() == ViewportTabId);

	return SNew(SDockTab)
		.Label(LOCTEXT("TextureViewportTitle", "Viewport"))
		[
			TextureViewport.ToSharedRef()
		];
}


void FTextureEditorToolkit::HandleTextureBorderActionExecute( )
{
	UTextureEditorSettings& Settings = *GetMutableDefault<UTextureEditorSettings>();
	Settings.TextureBorderEnabled = !Settings.TextureBorderEnabled;
	Settings.PostEditChange();
}


bool FTextureEditorToolkit::HandleTextureBorderActionIsChecked( ) const
{
	const UTextureEditorSettings& Settings = *GetDefault<UTextureEditorSettings>();

	return Settings.TextureBorderEnabled;
}

EVisibility FTextureEditorToolkit::HandleExposureBiasWidgetVisibility() const
{
	if (Texture)
	{
		FTexturePlatformData** RunningPlatformDataPtr = Texture->GetRunningPlatformData();
		if (RunningPlatformDataPtr && *RunningPlatformDataPtr && IsHDR((*RunningPlatformDataPtr)->PixelFormat))
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}


TOptional<int32> FTextureEditorToolkit::HandleExposureBiasBoxValue() const
{
	return GetExposureBias();
}

void FTextureEditorToolkit::HandleExposureBiasBoxValueChanged(int32 NewExposure)
{
	ExposureBias = NewExposure;
}

void FTextureEditorToolkit::HandleOpacitySliderChanged(float NewValue)
{
	SetVolumeOpacity(NewValue);
}

TOptional<float> FTextureEditorToolkit::HandleOpacitySliderValue() const
{
	return GetVolumeOpacity();
}


FReply FTextureEditorToolkit::HandleViewOptionsMenuButtonClicked()
{
	if (ViewOptionsMenuAnchor->ShouldOpenDueToClick())
	{
		ViewOptionsMenuAnchor->SetIsOpen(true);
	}
	else
	{
		ViewOptionsMenuAnchor->SetIsOpen(false);
	}

	return FReply::Handled();
}

void FTextureEditorToolkit::HandleZoomMenuEntryClicked(double ZoomValue)
{
	SetCustomZoomLevel(ZoomValue);
}

void FTextureEditorToolkit::HandleZoomMenuFillClicked()
{
	SetZoomMode(ETextureEditorZoomMode::Fill);
}

void FTextureEditorToolkit::HandleZoomMenuFitClicked()
{
	SetZoomMode(ETextureEditorZoomMode::Fit);
}

bool FTextureEditorToolkit::IsZoomMenuFillChecked() const
{
	return IsCurrentZoomMode(ETextureEditorZoomMode::Fill);
}

bool FTextureEditorToolkit::IsZoomMenuFitChecked() const
{
	return IsCurrentZoomMode(ETextureEditorZoomMode::Fit);
}

FText FTextureEditorToolkit::HandleZoomPercentageText() const
{
	double DisplayedZoomLevel = CalculateDisplayedZoomLevel();
	FText ZoomLevelPercent = FText::AsPercent(DisplayedZoomLevel);

	// For fit and fill, show the effective zoom level in parenthesis - eg. "Fill (220%)"
	static const FText ZoomModeWithPercentFormat = LOCTEXT("ZoomModeWithPercentFormat", "{ZoomMode} ({ZoomPercent})");
	if (GetZoomMode() == ETextureEditorZoomMode::Fit)
	{
		static const FText ZoomModeFit = LOCTEXT("ZoomModeFit", "Fit");
		return FText::FormatNamed(ZoomModeWithPercentFormat, TEXT("ZoomMode"), ZoomModeFit, TEXT("ZoomPercent"), ZoomLevelPercent);
	}

	if (GetZoomMode() == ETextureEditorZoomMode::Fill)
	{
		static const FText ZoomModeFill = LOCTEXT("ZoomModeFill", "Fill");
		return FText::FormatNamed(ZoomModeWithPercentFormat, TEXT("ZoomMode"), ZoomModeFill, TEXT("ZoomPercent"), ZoomLevelPercent);
	}

	// If custom, then just the percent is enough
	return ZoomLevelPercent;
}

void FTextureEditorToolkit::HandleZoomSliderChanged(float SliderValue)
{
	// zoom slider is log scale, SliderValue in [0,1] between MinZoom and MaxZoom
	double Octaves = log2( MaxZoom/MinZoom );
	double ZoomValue = pow(2.0,SliderValue * Octaves) * MinZoom;

	SetCustomZoomLevel((float)ZoomValue);
}

float FTextureEditorToolkit::HandleZoomSliderValue() const
{
	double ZoomValue = CalculateDisplayedZoomLevel();
	double Octaves = log2( MaxZoom/MinZoom );
	double SliderValue = log2( ZoomValue/MinZoom ) / Octaves;

	return (float)SliderValue;
}

int32 FTextureEditorToolkit::GetEditorOodleSettingsEffort() const
{
	return CustomEncoding->OodleEncodeEffort;
}

void FTextureEditorToolkit::EditorOodleSettingsEffortChanged(int32 NewValue, ESelectInfo::Type SelectionType)
{
	bool bChanged = CustomEncoding->OodleEncodeEffort != NewValue;

	CustomEncoding->OodleEncodeEffort = IntCastChecked<uint8>(NewValue);

	if (CustomEncoding->bUseCustomEncode && bChanged)
	{
		PostTextureRecode();
	}
}

int32 FTextureEditorToolkit::GetEditorOodleSettingsTiling() const
{
	return CustomEncoding->OodleUniversalTiling;
}

void FTextureEditorToolkit::EditorOodleSettingsTilingChanged(int32 NewValue, ESelectInfo::Type SelectionType)
{
	bool bChanged = CustomEncoding->OodleUniversalTiling != NewValue;
	CustomEncoding->OodleUniversalTiling = IntCastChecked<uint8>(NewValue);

	if (CustomEncoding->bUseCustomEncode && bChanged)
	{
		PostTextureRecode();
	}
}

TOptional<int32> FTextureEditorToolkit::GetEditorOodleSettingsRDO() const
{
	return CustomEncoding->OodleRDOLambda;
}

void FTextureEditorToolkit::EditorOodleSettingsRDOCommitted(int32 NewValue, ETextCommit::Type CommitType)
{
	if (NewValue > 100)
	{
		NewValue = 100;
	}
	if (NewValue < 0)
	{
		NewValue = 0;
	}

	bool bChanged = CustomEncoding->OodleRDOLambda != (int8)NewValue;

	CustomEncoding->OodleRDOLambda = (int8)NewValue;

	if (CustomEncoding->bUseCustomEncode && bChanged)
	{
		PostTextureRecode();
	}
}


bool FTextureEditorToolkit::EditorOodleSettingsEnabled() const
{
	return CustomEncoding->bUseCustomEncode;
}

ECheckBoxState FTextureEditorToolkit::UseEditorOodleSettingsChecked() const
{
	return CustomEncoding->bUseCustomEncode ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FTextureEditorToolkit::OnUseEditorOodleSettingsChanged(ECheckBoxState NewState)
{
	// We need to convince the texture to recompress and signal all its users
	// that they need to update, so we do this by faking a compression method property change.
	CustomEncoding->bUseCustomEncode = NewState == ECheckBoxState::Checked ? true : false;

	PostTextureRecode();
}

TSharedRef<SWidget> FTextureEditorToolkit::MakeChannelControlWidget()
{
	auto OnChannelCheckStateChanged = [this](ECheckBoxState NewState, ETextureChannelButton Button)
	{
		OnChannelButtonCheckStateChanged(Button);
	};

	TSharedRef<SWidget> ChannelControl = 
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(2.0f)
		.AutoWidth()
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "TextureEditor.ChannelButtonStyle")
			.BorderBackgroundColor(this, &FTextureEditorToolkit::GetChannelButtonBackgroundColor, ETextureChannelButton::Red)
			.ForegroundColor(this, &FTextureEditorToolkit::GetChannelButtonForegroundColor, ETextureChannelButton::Red)
			.OnCheckStateChanged_Lambda(OnChannelCheckStateChanged, ETextureChannelButton::Red)
			.IsChecked(this, &FTextureEditorToolkit::OnGetChannelButtonCheckState, ETextureChannelButton::Red)
			.IsEnabled(this, &FTextureEditorToolkit::IsChannelButtonEnabled, ETextureChannelButton::Red)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("TextureEditor.ChannelButtonFont"))
				.Text(FText::FromString("R"))
			]
		]
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(2.0f)
		.AutoWidth()
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "TextureEditor.ChannelButtonStyle")
			.BorderBackgroundColor(this, &FTextureEditorToolkit::GetChannelButtonBackgroundColor, ETextureChannelButton::Green)
			.ForegroundColor(this, &FTextureEditorToolkit::GetChannelButtonForegroundColor, ETextureChannelButton::Green)
			.OnCheckStateChanged_Lambda(OnChannelCheckStateChanged, ETextureChannelButton::Green)
			.IsChecked(this, &FTextureEditorToolkit::OnGetChannelButtonCheckState, ETextureChannelButton::Green)
			.IsEnabled(this, &FTextureEditorToolkit::IsChannelButtonEnabled, ETextureChannelButton::Green)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("TextureEditor.ChannelButtonFont"))
				.Text(FText::FromString("G"))
			]
		]

		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(2.0f)
		.AutoWidth()
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "TextureEditor.ChannelButtonStyle")
			.BorderBackgroundColor(this, &FTextureEditorToolkit::GetChannelButtonBackgroundColor, ETextureChannelButton::Blue)
			.ForegroundColor(this, &FTextureEditorToolkit::GetChannelButtonForegroundColor, ETextureChannelButton::Blue)
			.OnCheckStateChanged_Lambda(OnChannelCheckStateChanged, ETextureChannelButton::Blue)
			.IsChecked(this, &FTextureEditorToolkit::OnGetChannelButtonCheckState, ETextureChannelButton::Blue)
			.IsEnabled(this, &FTextureEditorToolkit::IsChannelButtonEnabled, ETextureChannelButton::Blue)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("TextureEditor.ChannelButtonFont"))
				.Text(FText::FromString("B"))
			]
		]
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(2.0f)
		.AutoWidth()
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "TextureEditor.ChannelButtonStyle")
			.BorderBackgroundColor(this, &FTextureEditorToolkit::GetChannelButtonBackgroundColor, ETextureChannelButton::Alpha)
			.ForegroundColor(this, &FTextureEditorToolkit::GetChannelButtonForegroundColor, ETextureChannelButton::Alpha)
			.OnCheckStateChanged_Lambda(OnChannelCheckStateChanged, ETextureChannelButton::Alpha)
			.IsChecked(this, &FTextureEditorToolkit::OnGetChannelButtonCheckState, ETextureChannelButton::Alpha)
			.IsEnabled(this, &FTextureEditorToolkit::IsChannelButtonEnabled, ETextureChannelButton::Alpha)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("TextureEditor.ChannelButtonFont"))
				.Text(FText::FromString("A"))
			]
		];

	return ChannelControl;
}

TSharedRef<SWidget> FTextureEditorToolkit::MakeLODControlWidget()
{
	TSharedRef<SWidget> LODControl = SNew(SBox)
		.WidthOverride(212.0f)
		[
			SNew(SHorizontalBox)
			.IsEnabled(this, &FTextureEditorToolkit::HandleMipLevelCheckBoxIsEnabled)
			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f, 2.0f, 0.0f)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &FTextureEditorToolkit::HandleMipLevelCheckBoxIsChecked)
				.OnCheckStateChanged(this, &FTextureEditorToolkit::HandleMipLevelCheckBoxCheckedStateChanged)
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(SComboButton)
				.IsEnabled(this, &FTextureEditorToolkit::GetUseSpecifiedMip)
				.OnGetMenuContent(this, &FTextureEditorToolkit::OnGenerateMipMapLevelMenu)
				.ButtonContent()
				[
					SAssignNew(MipLevelTextBlock, STextBlock)
					.Text(FText::Format(LOCTEXT("MipLevel", "Mip Level {0}"), SpecifiedMipLevel))
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "TextureEditor.MipmapButtonStyle")
				.OnClicked(this, &FTextureEditorToolkit::HandleMipMapPlusButtonClicked)
				.IsEnabled(this, &FTextureEditorToolkit::GetUseSpecifiedMip)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Plus"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "TextureEditor.MipmapButtonStyle")
				.OnClicked(this, &FTextureEditorToolkit::HandleMipMapMinusButtonClicked)
				.IsEnabled(this, &FTextureEditorToolkit::GetUseSpecifiedMip)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Minus"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];

	return LODControl;
}

TSharedRef<SWidget> FTextureEditorToolkit::MakeLayerControlWidget()
{
	TSharedRef<SWidget> LayerControl = SNew(SBox)
		.WidthOverride(160.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f, 4.0f, 0.0f)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("TextureEditor", "Layer", "Layer"))
			]
			+ SHorizontalBox::Slot()
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				.VAlign(VAlign_Center)
			[
				SNew(SNumericEntryBox<int32>)
				.AllowSpin(true)
				.MinSliderValue(0)
				.MaxSliderValue(this, &FTextureEditorToolkit::GetMaxLayer)
				.Value(this, &FTextureEditorToolkit::HandleLayerEntryBoxValue)
				.OnValueChanged(this, &FTextureEditorToolkit::HandleLayerEntryBoxChanged)
			]
		];

	return LayerControl;
}

TSharedRef<SWidget> FTextureEditorToolkit::MakeSliceControlWidget()
{
	TSharedRef<SWidget> SliceControl = SNew(SBox)
		.WidthOverride(212.0f)
		[
			SNew(SHorizontalBox)
			.IsEnabled(this, &FTextureEditorToolkit::HandleSliceCheckBoxIsEnabled)
			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f, 4.0f, 0.0f)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("TextureEditor", "Slice", "Slice"))
			]
			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f, 2.0f, 0.0f)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &FTextureEditorToolkit::HandleSliceCheckBoxIsChecked)
				.OnCheckStateChanged(this, &FTextureEditorToolkit::HandleSliceCheckBoxCheckedStateChanged)
			]
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SNumericEntryBox<int32>)
				.IsEnabled(this, &FTextureEditorToolkit::GetUseSpecifiedSlice)
				.AllowSpin(true)
				.MinSliderValue(0)
				.MaxSliderValue(this, &FTextureEditorToolkit::GetMaxSlice)
				.Value(this, &FTextureEditorToolkit::HandleSliceEntryBoxValue)
				.OnValueChanged(this, &FTextureEditorToolkit::HandleSliceEntryBoxChanged)
			]
		];

	return SliceControl;
}

TSharedRef<SWidget> FTextureEditorToolkit::MakeFaceControlWidget()
{
	TSharedRef<SWidget> FaceControl = SNew(SBox)
		.WidthOverride(212.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f, 4.0f, 0.0f)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("TextureEditor", "Face", "Face"))
			]
			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f, 2.0f, 0.0f)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &FTextureEditorToolkit::HandleFaceCheckBoxIsChecked)
				.OnCheckStateChanged(this, &FTextureEditorToolkit::HandleFaceCheckBoxCheckedStateChanged)
			]
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SNumericEntryBox<int32>)
				.IsEnabled(this, &FTextureEditorToolkit::GetUseSpecifiedFace)
				.AllowSpin(true)
				.MinSliderValue(0)
				.MaxSliderValue(5)
				.Value(this, &FTextureEditorToolkit::HandleFaceEntryBoxValue)
				.OnValueChanged(this, &FTextureEditorToolkit::HandleFaceEntryBoxChanged)
			]
		];

	return FaceControl;
}

TSharedRef<SWidget> FTextureEditorToolkit::MakeExposureContolWidget()
{
	TSharedRef<SWidget> ExposureControl = SNew(SBox)
		.WidthOverride(160.0f)
		.Visibility(this, &FTextureEditorToolkit::HandleExposureBiasWidgetVisibility)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(8.0f, 0.0f, 4.0f, 0.0f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ExposureBiasLabel", "Exposure Bias"))
				]
				+ SHorizontalBox::Slot()
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SNumericEntryBox<int32>)
					.AllowSpin(true)
					.MinSliderValue(MinExposure)
					.MaxSliderValue(MaxExposure)
					.Value(this, &FTextureEditorToolkit::HandleExposureBiasBoxValue)
					.OnValueChanged(this, &FTextureEditorToolkit::HandleExposureBiasBoxValueChanged)
				]
			]
		];
	return ExposureControl;
}

TSharedRef<SWidget> FTextureEditorToolkit::MakeOpacityControlWidget()
{
	TSharedRef<SWidget> OpacityControl = SNew(SBox)
		.WidthOverride(160.0f)
		[
			// opacity slider
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("OpacityLabel", "Opacity"))
			]
			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SNumericEntryBox<float>)
				.AllowSpin(true)
				.MinSliderValue(0.0f)
				.MaxSliderValue(1.0f)
				.OnValueChanged(this, &FTextureEditorToolkit::HandleOpacitySliderChanged)
				.Value(this, &FTextureEditorToolkit::HandleOpacitySliderValue)
			]
		];

	return OpacityControl;
}


TSharedRef<SWidget> FTextureEditorToolkit::MakePlatformSelectorWidget()
{
	
	if (AvailablePlatforms.Num() == 0) // we can get called multiple times by slate for some reason.
	{
		FProjectStatus ProjectStatus;
		bool bProjectStatusIsValid = IProjectManager::Get().QueryStatusForCurrentProject(ProjectStatus);

		AvailablePlatforms.Add(MakeShared<FString>(LOCTEXT("TextureViewEditorPlatform", "Editor Platform").ToString()));
		AvailablePlatformNames.Add(NAME_None);
		
		for (const auto& Pair : FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos())
		{
			if (Pair.Value.bIsFakePlatform || Pair.Value.bEnabledForUse == false)
			{
				continue;
			}

			FName PlatformName = Pair.Key;
			const FDataDrivenPlatformInfo& Info = Pair.Value;
			if (FDataDrivenPlatformInfoRegistry::IsPlatformHiddenFromUI(PlatformName))
			{
				continue;
			}

			if (!FDataDrivenPlatformInfoRegistry::HasCompiledSupportForPlatform(PlatformName, FDataDrivenPlatformInfoRegistry::EPlatformNameType::Ini))
			{
				continue;
			}

			if (bProjectStatusIsValid && !ProjectStatus.IsTargetPlatformSupported(PlatformName))
			{
				continue;
			}

			const PlatformInfo::FTargetPlatformInfo* VanillaInfo = PlatformInfo::FindVanillaPlatformInfo(Pair.Key);
			// VanillaInfo can come back null for platforms that are partially valid
			if ( VanillaInfo == nullptr )
			{
				continue;
			}

			const TArray<const PlatformInfo::FTargetPlatformInfo*> ValidFlavors = VanillaInfo->Flavors.FilterByPredicate([](const PlatformInfo::FTargetPlatformInfo* Target)
				{
					// Editor isn't a valid platform type that users can target
					// The Build Target will choose client or server, so no need to show them as well
					return Target->PlatformType != EBuildTargetType::Editor && Target->PlatformType != EBuildTargetType::Client && Target->PlatformType != EBuildTargetType::Server;
				});

			if (ValidFlavors.Num())
			{
				for (const PlatformInfo::FTargetPlatformInfo* TPI : ValidFlavors)
				{
					AvailablePlatforms.Add(MakeShared<FString>(TPI->DisplayName.ToString()));
					AvailablePlatformNames.Add(TPI->Name);
				}
			}
			else
			{
				AvailablePlatforms.Add(MakeShared<FString>(PlatformName.ToString()));
				AvailablePlatformNames.Add(PlatformName);
			}
		}
	}

	TSharedPtr InitialSelection = AvailablePlatforms[0];
	if (Texture->OverrideRunningPlatformName != NAME_None)
	{
		for (int32 PlatIndex = 1; PlatIndex < AvailablePlatforms.Num(); PlatIndex++)
		{
			if (AvailablePlatformNames[PlatIndex] == Texture->OverrideRunningPlatformName)
			{
				InitialSelection = AvailablePlatforms[PlatIndex];
				ViewingPlatform = AvailablePlatformNames[PlatIndex];
				break;
			}
		}
	}

	TSharedRef<SWidget> PreviewPlatformControl =
		SNew(SBox)
		.WidthOverride(250.f)
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("PlatformPreviewLabel", "Preview Platform:"))
						.ToolTipText(LOCTEXT("PlatformPreviewTT", "If a platform is chosen, the texture will be encoded as though cooked for that platform, then if necessary decoded so that it can be viewed on this platform. Requires: Texture source, non-virtual texture, and GPU availability."))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(STextComboBox)
						.OptionsSource(&AvailablePlatforms)
						.InitiallySelectedItem(InitialSelection)
						.IsEnabled(this, &FTextureEditorToolkit::CanPlatformPreview)
						.OnSelectionChanged_Lambda(
							[this]
							(TSharedPtr<FString> NewPlatformName, ESelectInfo::Type SelectInfo)
						{
							int32 Index = AvailablePlatforms.Find(NewPlatformName);
							if (Index != INDEX_NONE)
							{
								ViewingPlatform = AvailablePlatformNames[Index];
								if( ViewingPlatform != Texture->OverrideRunningPlatformName )
								{
									Texture->BlockOnAnyAsyncBuild(); // PreEditChange , but don't mark as modified ; same as Modify(false)
									Texture->OverrideRunningPlatformName = ViewingPlatform;
									Texture->PostEditChange();
								}
							}
						})
				]
		];

	return PreviewPlatformControl;
}

TSharedRef<SWidget> FTextureEditorToolkit::MakeZoomControlWidget()
{
	const FMargin ToolbarSlotPadding(4.0f, 1.0f);
	const FMargin ToolbarButtonPadding(4.0f, 0.0f);

	FMenuBuilder ZoomMenuBuilder(true, NULL);
	{
		FUIAction Zoom25Action(FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleZoomMenuEntryClicked, 0.25));
		ZoomMenuBuilder.AddMenuEntry(LOCTEXT("Zoom25Action", "25%"), LOCTEXT("Zoom25ActionHint", "Show the texture at a quarter of its size."), FSlateIcon(), Zoom25Action);

		FUIAction Zoom50Action(FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleZoomMenuEntryClicked, 0.5));
		ZoomMenuBuilder.AddMenuEntry(LOCTEXT("Zoom50Action", "50%"), LOCTEXT("Zoom50ActionHint", "Show the texture at half its size."), FSlateIcon(), Zoom50Action);

		FUIAction Zoom100Action(FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleZoomMenuEntryClicked, 1.0));
		ZoomMenuBuilder.AddMenuEntry(LOCTEXT("Zoom100Action", "100%"), LOCTEXT("Zoom100ActionHint", "Show the texture in its original size."), FSlateIcon(), Zoom100Action);

		FUIAction Zoom200Action(FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleZoomMenuEntryClicked, 2.0));
		ZoomMenuBuilder.AddMenuEntry(LOCTEXT("Zoom200Action", "200%"), LOCTEXT("Zoom200ActionHint", "Show the texture at twice its size."), FSlateIcon(), Zoom200Action);

		FUIAction Zoom400Action(FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleZoomMenuEntryClicked, 4.0));
		ZoomMenuBuilder.AddMenuEntry(LOCTEXT("Zoom400Action", "400%"), LOCTEXT("Zoom400ActionHint", "Show the texture at four times its size."), FSlateIcon(), Zoom400Action);

		ZoomMenuBuilder.AddMenuSeparator();

		FUIAction ZoomFitAction(
			FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleZoomMenuFitClicked),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FTextureEditorToolkit::IsZoomMenuFitChecked)
		);
		ZoomMenuBuilder.AddMenuEntry(LOCTEXT("ZoomFitAction", "Scale To Fit"), LOCTEXT("ZoomFitActionHint", "Scales the texture down to fit within the viewport if needed."), FSlateIcon(), ZoomFitAction, NAME_None, EUserInterfaceActionType::RadioButton);

		FUIAction ZoomFillAction(
			FExecuteAction::CreateSP(this, &FTextureEditorToolkit::HandleZoomMenuFillClicked),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FTextureEditorToolkit::IsZoomMenuFillChecked)
		);
		ZoomMenuBuilder.AddMenuEntry(LOCTEXT("ZoomFillAction", "Scale To Fill"), LOCTEXT("ZoomFillActionHint", "Scales the texture up and down to fill the viewport."), FSlateIcon(), ZoomFillAction, NAME_None, EUserInterfaceActionType::RadioButton);
	}

	// zoom slider
	TSharedRef<SWidget> ZoomControl = 
		SNew(SBox)
		.WidthOverride(250.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ZoomLabel", "Zoom"))
			]
			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.WidthOverride(200.f)
				[
					SNew(SSlider)
					.OnValueChanged(this, &FTextureEditorToolkit::HandleZoomSliderChanged)
					.Value(this, &FTextureEditorToolkit::HandleZoomSliderValue)
				]
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[

					SNew(SComboButton)
					.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
					.ButtonContent()
					[
						SNew(STextBlock)
						.Text(this, &FTextureEditorToolkit::HandleZoomPercentageText)
					]
					.MenuContent()
					[
						ZoomMenuBuilder.MakeWidget()
					]
			]
		];

	return ZoomControl;
}

TSharedRef<SWidget> FTextureEditorToolkit::MakeView3DControlWidget()
{
	// currently both volume textures and cubemaps have only 2 view modes, one of which can be described as "3D View"
	// ("Trace Into Volume" for volume textures and "3D View" for cubemaps),
	// therefore view mode for both volume textures and cubemaps can be controlled using the same "3D View" checkbox widget.
	TSharedRef<SWidget> View3DControl = SNew(SBox)
		.WidthOverride(80.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f, 4.0f, 0.0f)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("TextureEditor", "3D View", "3D View"))
			]
			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f, 2.0f, 0.0f)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &FTextureEditorToolkit::HandleView3DCheckBoxIsChecked)
				.OnCheckStateChanged(this, &FTextureEditorToolkit::HandleView3DCheckBoxCheckedStateChanged)
			]
		];

	return View3DControl;
}

void FTextureEditorToolkit::OnEstimateCompressionChanged(ECheckBoxState NewState)
{
	OodleCompressedPreviewDDCKey.Set<FString>(FString());
	bEstimateCompressionEnabled = NewState == ECheckBoxState::Checked;
}
ECheckBoxState FTextureEditorToolkit::EstimateCompressionChecked() const
{
	return bEstimateCompressionEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FTextureEditorToolkit::PackagingSettingsChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	if (Selection.IsValid())
	{
		UProjectPackagingSettings const* ProjectSettings = GetDefault<UProjectPackagingSettings>();
		int32 CompressionLevelFromSettings = (int32)FOodleDataCompression::ECompressionLevel::Optimal3;
		if (*Selection == TEXT("DebugDevelopment"))
		{
			CompressionLevelFromSettings = ProjectSettings->PackageCompressionLevel_DebugDevelopment;
		}
		else if (*Selection == TEXT("TestShipping"))
		{
			CompressionLevelFromSettings = ProjectSettings->PackageCompressionLevel_TestShipping;
		}
		else if (*Selection == TEXT("Distribution"))
		{
			CompressionLevelFromSettings = ProjectSettings->PackageCompressionLevel_Distribution;
		}

		if (IntFitsIn<int8>(CompressionLevelFromSettings) == false)
		{
			UE_LOGF(LogTextureEditor, Warning, "PackageCompressionLevel_%ls in Project Settings is an invalid value = %d, must be [-128, 127]", **Selection, CompressionLevelFromSettings);
			CompressionLevelFromSettings = (int32)FOodleDataCompression::ECompressionLevel::Optimal3;
		}

		FOodleDataCompression::ECompressionLevel OldLevel = OodleCompressionLevel;
		FOodleDataCompression::ECompressionLevelFromValue(static_cast<int8>(CompressionLevelFromSettings), OodleCompressionLevel);

		const TCHAR* LevelName;
		FOodleDataCompression::ECompressionLevelToString(OodleCompressionLevel, &LevelName);
		OodleLevelUsed->SetText(FText::FromString(FString::Printf(TEXT("%s (%d)"), LevelName, CompressionLevelFromSettings)));

		if (OldLevel != OodleCompressionLevel)
		{
			OodleCompressedPreviewDDCKey.Set<FString>(FString());
		}
	}
}

FReply FTextureEditorToolkit::DetectSourceAlphaButton_Clicked()
{
	if (!Texture ||
		!Texture->Source.IsValid() ||
		!Texture->IsAsyncCacheComplete())
	{
		return FReply::Handled();
	}

	Texture->PreEditChange(nullptr);
	Texture->Source.UpdateChannelLinearMinMax();
	Texture->PostEditChange();
	OodleCompressedPreviewDDCKey.Set<FString>(FString());
	PopulateQuickInfo();
	return FReply::Handled();
}

bool FTextureEditorToolkit::CanPlatformPreview() const
{
	return Texture && 
		!Texture->VirtualTextureStreaming &&
		Texture->Availability == ETextureAvailability::GPU &&
		CanRecompressTexture(Texture);
}

#undef LOCTEXT_NAMESPACE
