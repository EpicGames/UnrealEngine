// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DerivedDataCacheKeyProxy.h"
#include "UObject/GCObject.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "EditorUndoClient.h"
#include "Toolkits/IToolkitHost.h"
#include "Interfaces/ITextureEditorToolkit.h"
#include "IDetailsView.h"
#include "TextureEditorSettings.h"

template <typename ItemType> class SListView;
class SMenuAnchor;
class STextBlock;
class SButton;
class STextureEditorViewport;
class UFactory;
class UTexture;

namespace FOodleDataCompression { enum class ECompressor : uint8; enum class ECompressionLevel : int8; }

enum class ETextureChannelButton : uint8
{
	Red,
	Green,
	Blue,
	Alpha
};

/**
 * Implements an Editor toolkit for textures.
 */
class FTextureEditorToolkit
	: public ITextureEditorToolkit
	, public FEditorUndoClient
	, public FGCObject
{
public:
	FTextureEditorToolkit();

	/**
	 * Destructor.
	 */
	virtual ~FTextureEditorToolkit( );

public:

	/**
	 * Edits the specified Texture object.
	 *
	 * @param Mode The tool kit mode.
	 * @param InitToolkitHost 
	 * @param ObjectToEdit The texture object to edit.
	 */
	void InitTextureEditor( const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, UObject* ObjectToEdit );

public:
	// FAssetEditorToolkit interface
	virtual FString GetDocumentationLink( ) const override;
	virtual void RegisterTabSpawners( const TSharedRef<class FTabManager>& TabManager ) override;
	virtual void UnregisterTabSpawners( const TSharedRef<class FTabManager>& TabManager ) override;

	// ITextureEditorToolkit interface
	virtual void CalculateTextureDimensions(int32& OutWidth, int32& OutHeight, int32& OutDepth, int32& OutArraySize, bool bInIncludeBorderSize) const override;
	virtual ESimpleElementBlendMode GetColourChannelBlendMode( ) const override;
	virtual int32 GetMipLevel( ) const override;
	virtual int32 GetLayer() const override;
	virtual int32 GetSlice() const override;
	virtual int32 GetFace() const override;
	virtual UTexture* GetTexture( ) const override;
	virtual bool HasValidTextureResource( ) const override;
	virtual bool GetUseSpecifiedMip( ) const override;
	virtual bool GetUseSpecifiedSlice() const override;
	virtual bool GetUseSpecifiedFace() const override;
	virtual double GetCustomZoomLevel( ) const override;
	virtual void SetCustomZoomLevel(double ZoomValue) override;
	virtual void PopulateQuickInfo( ) override;
	virtual void SetZoomMode( const ETextureEditorZoomMode ZoomMode ) override;
	virtual ETextureEditorZoomMode GetZoomMode() const override;
	virtual double CalculateDisplayedZoomLevel() const override;
	virtual void ZoomIn( ) override;
	virtual void ZoomOut( ) override;
	virtual float GetVolumeOpacity( ) const override;
	virtual void SetVolumeOpacity( float VolumeOpacity ) override;
	virtual ETextureEditorVolumeViewMode GetVolumeViewMode() const override;
	virtual void SetVolumeViewMode(const ETextureEditorVolumeViewMode VolumeViewMode) override;
	virtual ETextureEditorCubemapViewMode GetCubemapViewMode() const override;
	virtual void SetCubemapViewMode(const ETextureEditorCubemapViewMode CubemapViewMode) override;
	virtual bool IsUsingOrientation() const override;
	virtual const FRotator& GetOrientation() const override;
	virtual void SetOrientation(const FRotator& InOrientation) override;
	virtual void ResetOrientation() override;
	virtual ETextureEditorSampling GetSampling() const override;
	virtual int32 GetExposureBias() const override
	{
		return ExposureBias;
	}
	virtual EVisibility GetVisibilityWhileAssetCompiling() const override
	{
		return EVisibility::Visible;
	}

	// IToolkit interface
	virtual FText GetBaseToolkitName( ) const override;
	virtual FName GetToolkitFName( ) const override;
	virtual FLinearColor GetWorldCentricTabColorScale( ) const override;
	virtual FString GetWorldCentricTabPrefix( ) const override;

	// FGCObject interface
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FTextureEditorToolkit");
	}

protected:
	// FEditorUndoClient interface
	virtual void PostUndo( bool bSuccess ) override;
	virtual void PostRedo( bool bSuccess ) override;

	/**
	 * Binds the UI commands to delegates.
	 */
	void BindCommands( );

	/**
	 * Creates the texture properties details widget.
	 *
	 * @return The widget.
	 */
	TSharedRef<SWidget> BuildTexturePropertiesWidget( );

	/**
	 * Creates all internal widgets for the tabs to point at.
	 */
	void CreateInternalWidgets( );

	/**
	 * Builds the toolbar widget for the Texture editor.
	 */
	void ExtendToolBar( );

	void FillToolbar(FToolBarBuilder& ToolbarBuilder);
	/**
	 * Gets the highest mip map level that this texture supports.
	 *
	 * @return Mip map level.
	 */
	TOptional<int32> GetMaxMipLevel( ) const;

	/**
	 * Gets the number of mips that this texture supports.
	 */
	int32 GetNumMips() const;

	/**
	 * Gets the texture format of this texture.
	 */
	EPixelFormat GetPixelFormat() const;

	/**
	 * Gets the highest layer index for this texture.
	 */
	TOptional<int32> GetMaxLayer() const;

	/**
	 * Gets the highest slice index for this texture.
	 */
	TOptional<int32> GetMaxSlice() const;

	/**
	 * Checks whether the texture being edited is a cube map texture.
	 */
	bool IsCubeTexture( ) const;

	/**
	 * Checks whether the texture being edited is a volume texture.
	 */
	bool IsVolumeTexture() const;

	/**
	 * Checks whether the texture being edited is a texture 2D array.
	 */
	bool Is2DArrayTexture() const;

	/**
	 * Checks whether the texture being edited is a texture array (Cube or 2D).
	 */
	bool IsArrayTexture() const;

	TSharedRef<SWidget> OnGenerateMipMapLevelMenu();
	TSharedRef<SWidget> OnGenerateSettingsMenu();
private:
	bool IsChannelButtonEnabled(ETextureChannelButton Button) const;
	FSlateColor GetChannelButtonBackgroundColor(ETextureChannelButton Button) const;
	FSlateColor GetChannelButtonForegroundColor(ETextureChannelButton Button) const;
	void OnChannelButtonCheckStateChanged(ETextureChannelButton Button);
	ECheckBoxState OnGetChannelButtonCheckState(ETextureChannelButton Button) const;

	// Callback for toggling the Checkered Background action.
	void HandleCheckeredBackgroundActionExecute( ETextureEditorBackgrounds Background );

	// Callback for getting the checked state of the Checkered Background action.
	bool HandleCheckeredBackgroundActionIsChecked( ETextureEditorBackgrounds Background );

	// Callback for toggling the Sampling action.
	void HandleSamplingActionExecute(ETextureEditorSampling Sampling);

	// Callback for getting the checked state of the Sampling action.
	bool HandleSamplingActionIsChecked(ETextureEditorSampling Sampling);

	// Callback for toggling the volume view action.
	void HandleVolumeViewModeActionExecute( ETextureEditorVolumeViewMode InViewMode );

	// Callback for getting the checked state of the volume view action.
	bool HandleVolumeViewModeActionIsChecked( ETextureEditorVolumeViewMode InViewMode );

	// Callback for toggling the cubemap view action.
	void HandleCubemapViewModeActionExecute(ETextureEditorCubemapViewMode InViewMode);

	// Callback for getting the checked state of the cubemap view action.
	bool HandleCubemapViewModeActionIsChecked(ETextureEditorCubemapViewMode InViewMode);

	// Callback for toggling the Compress Now action.
	void HandleCompressNowActionExecute( );

	// Callback for getting the checked state of the Compress Now action.
	bool HandleCompressNowActionCanExecute( ) const;

	// Callback for executing the Fit To Viewport action.
	void HandleFitToViewportActionExecute( );

	// Callback for executing the Fill To Viewport action.
	void HandleFillToViewportActionExecute( );

	virtual bool IsFitToViewport() const;
	virtual bool IsFillToViewport() const;
	
	// Callback for executing the Fit To 100% action.
	void HandleZoomToNaturalActionExecute( );

	// Callback for changing the checked state of the MipMap check box.
	void HandleMipLevelCheckBoxCheckedStateChanged( ECheckBoxState InNewState );

	// Callback for getting the checked state of the MipMap check box.
	ECheckBoxState HandleMipLevelCheckBoxIsChecked( ) const;

	// Callback for determining whether the MipMap check box is enabled.
	bool HandleMipLevelCheckBoxIsEnabled( ) const;

	// Callback for changing the value of the mip map level entry box.
	void HandleMipLevelChanged( int32 NewMipLevel );

	// Callback for getting the value of the mip map level entry box.
	TOptional<int32> HandleMipLevelEntryBoxValue( ) const;

	FReply HandleMipMapMinusButtonClicked();

	FReply HandleMipMapPlusButtonClicked();

	void HandleLayerEntryBoxChanged(int32 NewMipLevel);
	TOptional<int32> HandleLayerEntryBoxValue() const;

	bool HasLayers() const;

	// Returns true if the current texture supports slice selection.
	bool HasSlices() const;

	// Gets the number of slices that can be selected for this texture.
	int32 GetNumSlices() const;

	// Callback for determining whether the slice check box is enabled.
	bool HandleSliceCheckBoxIsEnabled() const;

	// Callback for getting the checked state of the slice check box.
	ECheckBoxState HandleSliceCheckBoxIsChecked() const;

	// Callback for changing the checked state of the slice check box.
	void HandleSliceCheckBoxCheckedStateChanged(ECheckBoxState InNewState);

	// Callback for getting the value of the slice index entry box.
	TOptional<int32> HandleSliceEntryBoxValue() const;

	// Callback for changing the value of the slice index entry box.
	void HandleSliceEntryBoxChanged(int32 Slice);

	// Callback for getting the checked state of the face check box.
	ECheckBoxState HandleFaceCheckBoxIsChecked() const;

	// Callback for changing the checked state of the face check box.
	void HandleFaceCheckBoxCheckedStateChanged(ECheckBoxState InNewState);

	// Callback for getting the value of the face index entry box.
	TOptional<int32> HandleFaceEntryBoxValue() const;

	// Callback for changing the value of the face index entry box.
	void HandleFaceEntryBoxChanged(int32 Face);

	// Callback for getting the checked state of the 3D View check box.
	ECheckBoxState HandleView3DCheckBoxIsChecked() const;

	// Callback for changing the checked state of the 3D View check box.
	void HandleView3DCheckBoxCheckedStateChanged(ECheckBoxState InNewState);

	// Callback for determining whether the Reimport action can execute.
	bool HandleReimportActionCanExecute( ) const;

	// Callback for executing the Reimport action.
	void HandleReimportActionExecute( );

	// Callback that is executed after the reimport manager reimported an asset.
	void HandleReimportManagerPostReimport( UObject* InObject, bool bSuccess );
	
	// Callback that is executed before the reimport manager reimported an asset.
	void HandleReimportManagerPreReimport( UObject* InObject );

	// Callback that is executed once an asset is imported
	void HandleAssetPostImport(UFactory* InFactory, UObject* InObject);

	// Callback for toggling the Desaturation channel action.
	void HandleDesaturationChannelActionExecute( );

	// Callback for getting the checked state of the Desaturation channel action.
	bool HandleDesaturationChannelActionIsChecked( ) const;

	// Callback for determining whether the Settings action can execute.
	void HandleSettingsActionExecute( );

	// Tab spawner callbacks
	TSharedRef<SDockTab> HandleTabSpawnerSpawnProperties(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> HandleTabSpawnerSpawnViewport( const FSpawnTabArgs& Args );
	TSharedRef<SDockTab> HandleTabSpawnerSpawnOodle(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> HandleTabSpawnerSpawnPlatforms(const FSpawnTabArgs& Args);

	// Callback for toggling the Texture Border action.
	void HandleTextureBorderActionExecute( );

	// Callback for getting the checked state of the Texture Border action.
	bool HandleTextureBorderActionIsChecked( ) const;

	// Callback for getting the visibility of the exposure bias widget.
	EVisibility HandleExposureBiasWidgetVisibility() const;

	// Callback for getting the exposure bias.
	TOptional<int32> HandleExposureBiasBoxValue() const;

	// Callback for changing the exposure bias.
	void HandleExposureBiasBoxValueChanged(int32 NewExposure);

	// Callback for changes in the zoom slider.
	void HandleOpacitySliderChanged(float NewValue);

	// Callback for getting the zoom slider's value.
	TOptional<float> HandleOpacitySliderValue() const;

	// Callback for clicking an item in the 'Zoom' menu.
	void HandleZoomMenuEntryClicked(double ZoomValue);

	// Callback for clicking the 'Fill' item in the 'Zoom' menu.
	void HandleZoomMenuFillClicked();

	// Callback for setting the checked state of the 'Fill' item in the 'Zoom' menu.
	bool IsZoomMenuFillChecked() const;

	// Callback for clicking the 'Fit' item in the 'Zoom' menu.
	void HandleZoomMenuFitClicked();

	// Callback for setting the checked state of the 'Fit' item in the 'Zoom' menu.
	bool IsZoomMenuFitChecked() const;

	// Callback for getting the zoom percentage text.
	FText HandleZoomPercentageText() const;

	// Callback for changes in the zoom slider.
	void HandleZoomSliderChanged(float NewValue);

	// Callback for getting the zoom slider's value.
	float HandleZoomSliderValue() const;

	// Callback for clicking the View Options menu button.
	FReply HandleViewOptionsMenuButtonClicked();

	TSharedRef<SWidget> MakeChannelControlWidget();
	TSharedRef<SWidget> MakeLODControlWidget();
	TSharedRef<SWidget> MakeLayerControlWidget();
	TSharedRef<SWidget> MakeSliceControlWidget();
	TSharedRef<SWidget> MakeFaceControlWidget();
	TSharedRef<SWidget> MakeExposureContolWidget();
	TSharedRef<SWidget> MakeOpacityControlWidget();
	TSharedRef<SWidget> MakeZoomControlWidget();
	TSharedRef<SWidget> MakeView3DControlWidget();
	TSharedRef<SWidget> MakePlatformSelectorWidget();
private:

	/** The Texture asset being inspected */
	TObjectPtr<UTexture> Texture;

	/** Viewport */
	TSharedPtr<STextureEditorViewport> TextureViewport;

	/** Oodle tab */
	TSharedPtr<SVerticalBox> OodleTabContainer;
	TSharedPtr<SVerticalBox> PlatformsTabContainer;

	/** Properties tab */
	TSharedPtr<SVerticalBox> TextureProperties;

	/** Properties tree view */
	TSharedPtr<class IDetailsView> TexturePropertiesWidget;

	/** Quick info text blocks */
	TSharedPtr<STextBlock> ImportedText;
	TSharedPtr<STextBlock> CurrentText;
	TSharedPtr<STextBlock> MaxInGameText;
	TSharedPtr<STextBlock> SizeText;
	TSharedPtr<STextBlock> MethodText;
	TSharedPtr<STextBlock> FormatText;
	TSharedPtr<STextBlock> LODBiasText;
	TSharedPtr<STextBlock> HasAlphaChannelText;
	TSharedPtr<STextBlock> SourceMipsAlphaDetectedText;
	TSharedPtr<SButton> DetectSourceAlphaButton;
	TSharedPtr<STextBlock> NumMipsText;
	TSharedPtr<STextBlock> MipLevelTextBlock;
	TSharedPtr<STextBlock> EncodeSpeedText;
	TSharedPtr<STextBlock> SceneCaptureSizeText;
	TSharedPtr<STextBlock> SceneCaptureNameText;

	// oodle details text blocks.
	TSharedPtr<STextBlock> OodleRDOText;
	TSharedPtr<STextBlock> OodleEffortText;
	TSharedPtr<STextBlock> OodleTilingText;
	TSharedPtr<STextBlock> OodleRDOSourceText;
	TSharedPtr<STextBlock> OodleEncoderText;
	TSharedPtr<STextBlock> OodleEncodeSpeedText;

	// oodle details labels
	TSharedPtr<STextBlock> OodleRDOEnabledLabel;
	TSharedPtr<STextBlock> OodleRDOSourceLabel;
	TSharedPtr<STextBlock> OodleEffortLabel;
	TSharedPtr<STextBlock> OodleTilingLabel;

	// Holds the anchor for the view options menu.
	TSharedPtr<SMenuAnchor> ViewOptionsMenuAnchor;

	/** If true, displays the red channel */
	bool bIsRedChannel;

	/** If true, displays the green channel */
	bool bIsGreenChannel;

	/** If true, displays the blue channel */
	bool bIsBlueChannel;

	/** If true, displays the alpha channel */
	bool bIsAlphaChannel;

	/** If true, desaturates the texture */
	bool bIsDesaturation;

	/** The maximum width/height at which the texture will render in the preview window */
	int32 PreviewEffectiveTextureWidth;
	int32 PreviewEffectiveTextureHeight;

	/** Which mip level should be shown */
	int32 SpecifiedMipLevel;

	/** When true, the specified mip value is used. Top mip is used when false.*/
	bool bUseSpecifiedMipLevel;

	/** Which layer should be shown */
	int32 SpecifiedLayer;

	/** Which slice should be shown */
	int32 SpecifiedSlice;

	/** When true, the specified slice index value is used, otherwise the value of -1 is used.*/
	bool bUseSpecifiedSlice;

	/** Which cubemap face should be shown */
	int32 SpecifiedFace;

	/** When true, the specified cubemap face index value is used, otherwise the value of -1 is used.*/
	bool bUseSpecifiedFace;

	/** The texture's zoom factor. */
	double Zoom;

	FName ViewingPlatform; // NAME_None if viewing editor.

	// Which exposure level should be used, in FStop e.g. 0:original, -1:half as bright, 1:2x as bright, 2:4x as bright.
	int32 ExposureBias;

	/** This toolkit's current zoom mode **/
	ETextureEditorZoomMode ZoomMode;

	// For volume texture, defines an opacity to see through the volume when tracing.
	float VolumeOpacity;

	/** This toolkit's current volume view mode **/
	ETextureEditorVolumeViewMode VolumeViewMode;

	/** This toolkit's current cubemap view mode **/
	ETextureEditorCubemapViewMode CubemapViewMode;

	/** This toolkit's current sampling mode **/
	ETextureEditorSampling Sampling;

	// Orientation of the texture in 3d when using "3D View" mode
	FRotator Orientation;

	bool bIsVolumeTexture;

	TArray<TSharedPtr<FString>> AvailablePlatforms;
	TArray<FName> AvailablePlatformNames;

public:
	struct FPlatformTextureFormats
	{
		FName PlatformName;
		FName TextureFormatName;

		// Pixel formats for whether we detected alpha in the texture source. Only
		// matters for Auto* texture formats. If we don't know the texture source alpha
		// information we show both.
		EPixelFormat PixelFormat_AlphaTrue = PF_Unknown;
		EPixelFormat PixelFormat_AlphaFalse = PF_Unknown;
		bool bKnowAlphaTransparency = false;
		bool bHasTransparentAlpha = false;
		bool bIsVirtual = false;
		int32 Width = 0;
		int32 Height = 0;
		int32 NumMips_TotalAfterLOD = 0;
		int32 NumMips_Optional = 0;
		int32 NumMips_Streaming = 0;
		int32 NumMips_Tail = 0;
		int8 LODBias = 0;
		uint64 MipChainSizeInBytes_Streaming = 0;
		uint64 MipChainSizeInBytes_Inline = 0;
		uint64 MipChainSizeInBytes_Optional = 0;
		TArray<uint64, TInlineAllocator<MAX_TEXTURE_MIP_COUNT>> MipSizesInBytes;
	};
private:

	TSharedPtr<SListView<TSharedPtr<FPlatformTextureFormats>>> PlatformTextureFormatsWidget;
	TArray<TSharedPtr<FPlatformTextureFormats>> PlatformTextureFormatList;
	TSharedRef<ITableRow> MakePlatformTextureFormatWidget(TSharedPtr<FPlatformTextureFormats> PlatformTextureFormat, const TSharedRef<STableViewBase>& OwnerTable);

	TMap<FName, TPair<TSharedPtr<STextBlock>, TSharedPtr<STextBlock>>> PlatformFormatDetails;

	// Objects and callbacks for the custom encoding settings checkbox
	TSharedPtr<class SCheckBox> OodleOverrideCheck;
	void OnUseEditorOodleSettingsChanged(ECheckBoxState NewState);
	ECheckBoxState UseEditorOodleSettingsChecked() const;

	FReply DetectSourceAlphaButton_Clicked();
	bool CanPlatformPreview() const;

	// callback for enabling custom encoding sub-controls
	bool EditorOodleSettingsEnabled() const;

	// accessor callbacks for custom encoding sub controls
	int32 GetEditorOodleSettingsEffort() const;
	void EditorOodleSettingsEffortChanged(int32 NewValue, ESelectInfo::Type SelectionType);
	TOptional<int32> GetEditorOodleSettingsRDO() const;
	void EditorOodleSettingsRDOCommitted(int32 NewValue, ETextCommit::Type CommitType);
	int32 GetEditorOodleSettingsTiling() const;
	void EditorOodleSettingsTilingChanged(int32 NewValue, ESelectInfo::Type SelectionType);

	// This is the key we have compressed for oodle preview, so we can catch changes.
	TVariant<FString, UE::DerivedData::FCacheKeyProxy> OodleCompressedPreviewDDCKey;

	
	//
	// -- Start on disc estimation controls/fns
	//
	bool bEstimateCompressionEnabled = false;
	bool EstimateCompressionEnabled() const { return bEstimateCompressionEnabled; }

	// Parsed from packaging settings.
	FOodleDataCompression::ECompressionLevel OodleCompressionLevel;
	FOodleDataCompression::ECompressor OodleCompressor;
	uint32 CompressionBlockSize;

	// Valid if we've kicked off an estimation task and are waiting for the result.
	TFuture<TTuple<uint64, uint64>> OutstandingEstimation;

	// Controls
	TSharedPtr<class SCheckBox> OodleEstimateCheck;
	void OnEstimateCompressionChanged(ECheckBoxState NewState);
	ECheckBoxState EstimateCompressionChecked() const;

	TSharedPtr<STextBlock> OodleCompressorUsed;
	TSharedPtr<STextBlock> OodleLevelUsed;
	TSharedPtr<STextBlock> OodleCompressionBlockUsed;
	TSharedPtr<STextBlock> OodleEstimateRaw;
	TSharedPtr<STextBlock> OodleEstimateCompressed;

	// For the packaging settings combo box 
	TArray< TSharedPtr< FString > > PackagingSettingsNames;
	void PackagingSettingsChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);
	//
	// -- End on disc estimation controls/fns
	//

	void PostTextureRecode();
public:
	// These are on a separate object so then engine can share the type without needing
	// our module.
	TSharedPtr<class FTextureEditorCustomEncode> CustomEncoding;

private:

	// Tab names
	static const FName ViewportTabId;
	static const FName PropertiesTabId;
	static const FName OodleTabId;
	static const FName PlatformsTabId;
};
