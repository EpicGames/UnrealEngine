// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/STextureEditorViewport.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SViewport.h"
#include "Slate/SceneViewport.h"
#include "Widgets/Input/SNumericEntryBox.h"

#define LOCTEXT_NAMESPACE "STextureEditorViewport"


/* STextureEditorViewport interface
 *****************************************************************************/

void STextureEditorViewport::AddReferencedObjects( FReferenceCollector& Collector )
{
	ViewportClient->AddReferencedObjects(Collector);
}

void STextureEditorViewport::Construct( const FArguments& InArgs, const TSharedRef<ITextureEditorToolkit>& InToolkit )
{
	bIsRenderingEnabled = true;
	ToolkitPtr = InToolkit;

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SOverlay)
			// viewport canvas
			+ SOverlay::Slot()
			.Padding(5.0f, 0.0f)
			[
				SAssignNew(ViewportWidget, SViewport)
				.EnableGammaCorrection(false)
				.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
				.ShowEffectWhenDisabled(false)
				.EnableBlending(true)
				.ToolTip(SNew(SToolTip).Text(this, &STextureEditorViewport::GetDisplayedResolution))
			]
			+ SOverlay::Slot()
			.HAlign(HAlign_Right)
			[
				// vertical scroll bar
				SAssignNew(TextureViewportVerticalScrollBar, SScrollBar)
				.Visibility(this, &STextureEditorViewport::HandleVerticalScrollBarVisibility)
				.OnUserScrolled(this, &STextureEditorViewport::HandleVerticalScrollBarScrolled)
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			// horizontal scrollbar
			SAssignNew(TextureViewportHorizontalScrollBar, SScrollBar)
				.Orientation( Orient_Horizontal )
				.Visibility(this, &STextureEditorViewport::HandleHorizontalScrollBarVisibility)
				.OnUserScrolled(this, &STextureEditorViewport::HandleHorizontalScrollBarScrolled)
		]

	];
	
	ViewportClient = MakeShareable(new FTextureEditorViewportClient(ToolkitPtr, SharedThis(this)));

	Viewport = FSceneViewport::Create(ViewportClient, ViewportWidget);
}

void STextureEditorViewport::ModifyCheckerboardTextureColors( )
{
	if (ViewportClient.IsValid())
	{
		ViewportClient->ModifyCheckerboardTextureColors();
	}
}

void STextureEditorViewport::EnableRendering()
{
	bIsRenderingEnabled = true;
}

void STextureEditorViewport::DisableRendering()
{
	bIsRenderingEnabled = false;
}

TSharedPtr<FSceneViewport> STextureEditorViewport::GetViewport( ) const
{
	return Viewport;
}

TSharedPtr<SViewport> STextureEditorViewport::GetViewportWidget( ) const
{
	return ViewportWidget;
}

TSharedPtr<SScrollBar> STextureEditorViewport::GetVerticalScrollBar( ) const
{
	return TextureViewportVerticalScrollBar;
}

TSharedPtr<SScrollBar> STextureEditorViewport::GetHorizontalScrollBar( ) const
{
	return TextureViewportHorizontalScrollBar;
}


/* SWidget overrides
 *****************************************************************************/

void STextureEditorViewport::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if (bIsRenderingEnabled)
	{
		Viewport->Invalidate();
	}
}


/* STextureEditorViewport implementation
 *****************************************************************************/

FText STextureEditorViewport::GetDisplayedResolution( ) const
{
	return ViewportClient->GetDisplayedResolution();
}


/* STextureEditorViewport event handlers
 *****************************************************************************/

void STextureEditorViewport::HandleHorizontalScrollBarScrolled( float InScrollOffsetFraction )
{
	const float Ratio = ViewportClient->GetViewportHorizontalScrollBarRatio();
	const float MaxOffset = (Ratio < 1.0f) ? 1.0f - Ratio : 0.0f;
	InScrollOffsetFraction = FMath::Clamp(InScrollOffsetFraction, 0.0f, MaxOffset);

	TextureViewportHorizontalScrollBar->SetState(InScrollOffsetFraction, Ratio);
}

EVisibility STextureEditorViewport::HandleHorizontalScrollBarVisibility( ) const
{
	if (ViewportClient->GetViewportHorizontalScrollBarRatio() < 1.0f)
	{
		return EVisibility::Visible;
	}
	
	return EVisibility::Collapsed;
}

void STextureEditorViewport::HandleVerticalScrollBarScrolled( float InScrollOffsetFraction )
{
	const float Ratio = ViewportClient->GetViewportVerticalScrollBarRatio();
	const float MaxOffset = (Ratio < 1.0f) ? 1.0f - Ratio : 0.0f;
	InScrollOffsetFraction = FMath::Clamp(InScrollOffsetFraction, 0.0f, MaxOffset);

	TextureViewportVerticalScrollBar->SetState(InScrollOffsetFraction, Ratio);
}

EVisibility STextureEditorViewport::HandleVerticalScrollBarVisibility( ) const
{
	if (ViewportClient->GetViewportVerticalScrollBarRatio() < 1.0f)
	{
		return EVisibility::Visible;
	}
	
	return EVisibility::Collapsed;
}


#undef LOCTEXT_NAMESPACE
