// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Models/TextureEditorViewportClient.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Interfaces/ITextureEditorToolkit.h"

class FSceneViewport;
class SScrollBar;
class SViewport;

/**
 * Implements the texture editor's view port.
 */
class STextureEditorViewport
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(STextureEditorViewport) { }
	SLATE_END_ARGS()

	/**
	 */
	void AddReferencedObjects( FReferenceCollector& Collector );

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<ITextureEditorToolkit>& InToolkit );
	
	/**
	 * Modifies the checkerboard texture's data.
	 */
	void ModifyCheckerboardTextureColors( );


	/** Enable viewport rendering */
	void EnableRendering();

	/** Disable viewport rendering */
	void DisableRendering();

	TSharedPtr<FSceneViewport> GetViewport( ) const;
	TSharedPtr<SViewport> GetViewportWidget( ) const;
	TSharedPtr<SScrollBar> GetVerticalScrollBar( ) const;
	TSharedPtr<SScrollBar> GetHorizontalScrollBar( ) const;

public:

	// SWidget overrides

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

protected:

	/**
	 * Gets the displayed textures resolution as a string.
	 *
	 * @return Texture resolution string.
	 */
	FText GetDisplayedResolution() const;

private:

	// Callback for the horizontal scroll bar.
	void HandleHorizontalScrollBarScrolled( float InScrollOffsetFraction );

	// Callback for getting the visibility of the horizontal scroll bar.
	EVisibility HandleHorizontalScrollBarVisibility( ) const;

	// Callback for the vertical scroll bar.
	void HandleVerticalScrollBarScrolled( float InScrollOffsetFraction );

	// Callback for getting the visibility of the horizontal scroll bar.
	EVisibility HandleVerticalScrollBarVisibility( ) const;

	// Pointer back to the Texture editor tool that owns us.
	TWeakPtr<ITextureEditorToolkit> ToolkitPtr;
	
	// Level viewport client.
	TSharedPtr<class FTextureEditorViewportClient> ViewportClient;

	// Slate viewport for rendering and IO.
	TSharedPtr<FSceneViewport> Viewport;

	// Viewport widget.
	TSharedPtr<SViewport> ViewportWidget;

	// Vertical scrollbar.
	TSharedPtr<SScrollBar> TextureViewportVerticalScrollBar;

	// Horizontal scrollbar.
	TSharedPtr<SScrollBar> TextureViewportHorizontalScrollBar;

	// Is rendering currently enabled? (disabled when reimporting a texture)
	bool bIsRenderingEnabled = true;
};
