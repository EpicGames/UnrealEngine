// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Interfaces/ITextureEditorToolkit.h"
#include "Modules/ModuleInterface.h"

class UTexture;

/**
 * Interface for texture editor modules.
 */
class ITextureEditorModule
	: public IModuleInterface
	, public IHasMenuExtensibility
	, public IHasToolBarExtensibility
{
public:

	/**
	 * Creates a new Texture editor.
	 *
	 * @param Mode 
	 * @param InitToolkitHost 
	 * @param Texture 
	 */
	virtual TSharedRef<ITextureEditorToolkit> CreateTextureEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UTexture* Texture ) = 0;
};
