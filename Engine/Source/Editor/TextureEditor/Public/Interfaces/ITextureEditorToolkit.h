// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SceneTypes.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "TextureEditorSettings.h"

class UTexture;

/**
 * Interface for texture editor tool kits.
 */
class ITextureEditorToolkit
	: public FAssetEditorToolkit
{
public:

	/** Returns the Texture asset being inspected by the Texture editor */
	virtual UTexture* GetTexture() const = 0;

	/** Returns if the Texture asset being inspected has a valid texture resource */
	virtual bool HasValidTextureResource() const = 0;

	/** Refreshes the quick info panel */
	virtual void PopulateQuickInfo() = 0;

	/** Calculates the display size of the texture, optionally including the size of the displayed texture border */
	virtual void CalculateTextureDimensions(int32& OutWidth, int32& OutHeight, int32& OutDepth, int32& OutArraySize, bool bInIncludeBorderSize) const = 0;

	/** Accessors */ 
	virtual int32 GetMipLevel() const = 0;
	virtual int32 GetLayer() const = 0;
	virtual int32 GetSlice() const = 0;
	virtual int32 GetFace() const = 0;
	virtual ESimpleElementBlendMode GetColourChannelBlendMode() const = 0;
	virtual bool GetUseSpecifiedMip() const = 0;
	virtual bool GetUseSpecifiedSlice() const = 0;
	virtual bool GetUseSpecifiedFace() const = 0;
	virtual double GetCustomZoomLevel() const = 0;
	virtual void SetCustomZoomLevel( double ZoomValue ) = 0;
	virtual void ZoomIn() = 0;
	virtual void ZoomOut() = 0;
	virtual ETextureEditorZoomMode GetZoomMode() const = 0;
	virtual void SetZoomMode( const ETextureEditorZoomMode ZoomMode ) = 0;
	virtual double CalculateDisplayedZoomLevel() const = 0;
	virtual float GetVolumeOpacity( ) const = 0;
	virtual void SetVolumeOpacity( float VolumeOpacity ) = 0;
	virtual ETextureEditorVolumeViewMode GetVolumeViewMode() const = 0;
	virtual void SetVolumeViewMode(const ETextureEditorVolumeViewMode VolumeViewMode) = 0;
	virtual ETextureEditorCubemapViewMode GetCubemapViewMode() const = 0;
	virtual void SetCubemapViewMode(const ETextureEditorCubemapViewMode CubemapViewMode) = 0;
	virtual bool IsUsingOrientation() const = 0;
	virtual const FRotator& GetOrientation() const = 0;
	virtual void SetOrientation(const FRotator& InOrientation) = 0;
	virtual void ResetOrientation() = 0;
	virtual ETextureEditorSampling GetSampling() const = 0;
	virtual int32 GetExposureBias() const = 0;
	virtual bool IsVolumeTexture() const = 0;
public:

	/**
	 * Toggles the fit-to-viewport mode. If already on, will return to the last custom zoom level.
	 */
	UE_DEPRECATED(4.26, "There are now commands for switching to individual zoom modes rather than toggling. Please use SetZoomMode() instead.")
	void ToggleFitToViewport()
	{
		if (IsCurrentZoomMode(ETextureEditorZoomMode::Fit))
		{
			SetZoomMode(ETextureEditorZoomMode::Custom);
		}
		else
		{
			SetZoomMode(ETextureEditorZoomMode::Fit);
		}
	}

	/** 
	* Returns true if this is the current zoom mode. Useful for Slate bindings.
	*/
	bool IsCurrentZoomMode(ETextureEditorZoomMode ZoomMode) const
	{ 
		return GetZoomMode() == ZoomMode; 
	}
};
