// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/* Public Dependencies
 *****************************************************************************/

#include "CoreMinimal.h"


/* Public Includes
 *****************************************************************************/



extern const FName TextureEditorAppIdentifier;
