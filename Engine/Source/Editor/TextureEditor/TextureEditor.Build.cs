// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class TextureEditor : ModuleRules
{
	public TextureEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"MainFrame",
				"WorkspaceMenuStructure"
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Settings",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"ImageCore",
				"InputCore",
				"Engine",
				"Projects",
				"RenderCore",
				"RHI",
				"Slate",
				"SlateCore",
				"EditorFramework",
				"UnrealEd",
				"PropertyEditor",
				"EditorWidgets",
				"MediaAssets",
				"DerivedDataCache",
				"DesktopPlatform",
				"DeveloperToolSettings",
				"TextureCompressor",
				"TextureBuildUtilities"
			}
		);
	}
}
